{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Metasounds Experimental",
	"Description": "Metasound developmental plugin, for new features before they are ready for prime time",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioExperimentalRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MetasoundExperimentalRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MetasoundExperimentalEngineRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MetasoundExperimentalEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Metasound",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": true	
}