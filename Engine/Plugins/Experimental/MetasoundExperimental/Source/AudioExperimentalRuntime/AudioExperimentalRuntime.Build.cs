// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioExperimentalRuntime : ModuleRules
{
	public AudioExperimentalRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		// PLEASE DO NOT ADD AN ENGINE DEPENDENCY HERE!
		// HORDE DOES NOT CATCH IT EITHER
		// RUN (ushell): .build program AudioUnitTests OR build in VS
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AudioChannelAgnosticCore"
			}
		);
		
		// We want to use enum definitions in AudioMixer without an Engine dependency.
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"AudioMixer"
			}
		);
		
		// When the Editor builds all plugins, it will build with engine. In that case add a dependency on CoreUObject.
		// If we don't do this we will get linker errors for the Engine expecting to find stuff defined here.
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("CoreUObject");
		}
		
		// PLEASE DO NOT ADD AN ENGINE DEPENDENCY HERE!
		// HORDE DOES NOT CATCH IT EITHER
		// RUN (ushell): .build program AudioUnitTests OR build in VS
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"SignalProcessing",
			}
		);
	}
}