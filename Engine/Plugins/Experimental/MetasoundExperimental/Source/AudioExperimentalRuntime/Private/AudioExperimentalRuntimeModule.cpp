// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "TypeFamily/ChannelTypeFamily.h"

class FAudioCoreExperimentalModule final : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{}
	
	virtual void ShutdownModule() override
	{}
};
    
IMPLEMENT_MODULE(FAudioCoreExperimentalModule, AudioExperimentalRuntime)