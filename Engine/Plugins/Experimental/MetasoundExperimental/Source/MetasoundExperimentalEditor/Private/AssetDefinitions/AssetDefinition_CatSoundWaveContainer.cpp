// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinitions/AssetDefinition_CatSoundWaveContainer.h"

#include "Algo/AnyOf.h"
#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "CatSoundWaveContainerFactory.h"
#include "ContentBrowserMenuContexts.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProcedural.h"
#include "CatSoundWaveContainer.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FCatSoundWaveContainerExtension::RegisterMenus()
{
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
	FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
	check(Section);
	Section->AddDynamicEntry("SoundWaveAssetConversion_CreateCatSoundWaveContainer",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				if (!Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData) { return AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
				{
					const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateCatSoundWaveContainer", "Create Sound Wave Container");
					const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateCatSoundWaveContainerToolTip", "Creates a Sound Wave Container asset from the selected Sound Waves");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&FCatSoundWaveContainerExtension::Execute);

					InSection.AddMenuEntry("SoundWave_CreateCatSoundWaveContainer", Label, ToolTip, Icon, UIAction);
				}
			}
		}));
}

void FCatSoundWaveContainerExtension::Execute(const FToolMenuContext& MenuContext)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	UCatSoundWaveContainerFactory* Factory = NewObject<UCatSoundWaveContainerFactory>();

	TArray<TWeakObjectPtr<USoundWave>> StagedSoundWaves;
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		for (USoundWave* Wave : Context->LoadSelectedObjectsIf<USoundWave>([](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
		{
			StagedSoundWaves.Add(Wave);
		}

		if (StagedSoundWaves.Num() == 0)
		{
			return;
		}

		FString PackageName = StagedSoundWaves[0]->GetPackage()->GetName();
		FString Suffix = TEXT("");
		FString NewPackageName;
		FString NewName;
		AssetToolsModule.Get().CreateUniqueAssetName(PackageName, Suffix, NewPackageName, NewName);
		const FString PackagePath = FPackageName::GetLongPackagePath(NewPackageName);

		UObject* NewAsset = AssetToolsModule.Get().CreateAsset(NewName, PackagePath, UCatSoundWaveContainer::StaticClass(), Factory);

		if (UCatSoundWaveContainer* Container = Cast<UCatSoundWaveContainer>(NewAsset))
		{
			for (const TWeakObjectPtr<USoundWave>& SoundWave : StagedSoundWaves)
			{
				Container->Entries.Emplace(FCatSoundWaveContainerEntry(SoundWave.Get()));
			}

			TArray<UObject*> Assets = { NewAsset };
			FAssetToolsModule::GetModule().Get().SyncBrowserToAssets(Assets);
		}
	}
}

FText UAssetDefinition_CatSoundWaveContainer::GetAssetDisplayName() const
{
	return LOCTEXT("CatSoundWaveContainerDefinition", "Sound Wave Container");
}

FLinearColor UAssetDefinition_CatSoundWaveContainer::GetAssetColor() const
{
	return FLinearColor(FColor(80, 120, 255));
}

TSoftClassPtr<UObject> UAssetDefinition_CatSoundWaveContainer::GetAssetClass() const
{
	return UCatSoundWaveContainer::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CatSoundWaveContainer::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories =
	{
		FAssetCategoryPath(EAssetCategoryPaths::Audio, LOCTEXT("AssetDefinition_CatSoundWaveContainerSubMenu", "Experimental"))
	};
	return Categories;
}

bool UAssetDefinition_CatSoundWaveContainer::CanImport() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
