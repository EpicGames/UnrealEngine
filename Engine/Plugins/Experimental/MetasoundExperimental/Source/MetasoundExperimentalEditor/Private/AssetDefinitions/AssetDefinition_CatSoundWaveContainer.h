// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CatSoundWaveContainer.generated.h"

struct FToolMenuContext;

UCLASS(MinimalAPI)
class UAssetDefinition_CatSoundWaveContainer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override;
};

class FCatSoundWaveContainerExtension
{
public:
	static void RegisterMenus();
	static void Execute(const FToolMenuContext& MenuContext);
};
