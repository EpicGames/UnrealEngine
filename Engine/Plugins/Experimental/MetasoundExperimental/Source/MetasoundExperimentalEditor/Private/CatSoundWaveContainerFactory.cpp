// Copyright Epic Games, Inc. All Rights Reserved.

#include "CatSoundWaveContainerFactory.h"

#include "CatSoundWaveContainer.h"

UCatSoundWaveContainerFactory::UCatSoundWaveContainerFactory()
{
	SupportedClass = UCatSoundWaveContainer::StaticClass();
	bCreateNew = true;
	bText = false;
	bEditorImport = false;
}

UObject* UCatSoundWaveContainerFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* /*Context*/, FFeedbackContext* /*Warn*/)
{
	return NewObject<UCatSoundWaveContainer>(InParent, InName, Flags);
}
