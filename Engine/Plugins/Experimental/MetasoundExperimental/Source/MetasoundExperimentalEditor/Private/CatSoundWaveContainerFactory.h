// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CatSoundWaveContainerFactory.generated.h"

UCLASS(MinimalAPI)
class UCatSoundWaveContainerFactory : public UFactory
{
	GENERATED_BODY()

public:
	UCatSoundWaveContainerFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
