// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExampleNodeDetailsCustomization.h" 

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundExampleNodeConfiguration.h"
#include "MetasoundNodeConfigurationCustomization.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SSlider.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalEditor"

namespace MetasoundExampleNodeDetailsCustomizationPrivate
{
	const FString StructIdentifierWithDelimiters =  TEXT(".Struct.");
}

FExampleWidgetNodeConfigurationCustomization::FExampleWidgetNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
	: Metasound::Editor::FMetaSoundNodeConfigurationCustomization(InStructProperty, InNode)
{
	if (InStructProperty && InStructProperty->IsValidHandle())
	{
		StructPropertyPath = InStructProperty->GeneratePathToProperty();
	}
}

void FExampleWidgetNodeConfigurationCustomization::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	TSharedPtr<IPropertyHandle> ChildHandle = ChildRow.GetPropertyHandle();
	if (!ChildHandle || !ChildHandle->IsValidHandle())
	{
		return;
	}

	const FString PropertyPath = ChildHandle->GeneratePathToProperty();
	// Customize a specific member
	using namespace MetasoundExampleNodeDetailsCustomizationPrivate;
	if (PropertyPath == StructPropertyPath + StructIdentifierWithDelimiters + GET_MEMBER_NAME_CHECKED(FMetaSoundWidgetExampleNodeConfiguration, MyFloat).ToString())
	{
		MyFloatPropertyHandle = ChildHandle;

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow Row;

		ChildRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

		ChildRow.CustomWidget(true)
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SNew(SSlider)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Value_Lambda([Handle=ChildRow.GetPropertyHandle()]{
					float Value = 0.0f;
					if (Handle.IsValid() && Handle->IsValidHandle())
					{
						Handle->GetValue(Value);
					}
					return Value;
				})
				.OnValueChanged_Lambda([Handle = ChildRow.GetPropertyHandle()](float Value)
				{
					if (Handle.IsValid() && Handle->IsValidHandle())
					{
						Handle->SetValue(Value);
					}
				})
			];
	}
	// Add custom onvalue changed
	TDelegate<void(const FPropertyChangedEvent&)> OnValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FExampleWidgetNodeConfigurationCustomization::OnChildPropertyChanged);
	ChildHandle->SetOnPropertyValueChangedWithData(OnValueChangedDelegate);

	// Add base class on value changed
	Metasound::Editor::FMetaSoundNodeConfigurationCustomization::OnChildRowAdded(ChildRow);
}

void FExampleWidgetNodeConfigurationCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundWidgetExampleNodeConfiguration, MyFloat))
	{
		if (GraphNode.IsValid() && MyFloatPropertyHandle.IsValid() && MyFloatPropertyHandle->IsValidHandle())
		{
			const FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
			const FGuid& NodeID = GraphNode->GetNodeID();

			// Update the operator data value from the configuration property handle value
			// Node configuration operator data API is experimental
			// so this code will be made cleaner in the future 
			const TConstStructView<FMetaSoundFrontendNodeConfiguration>& Config = DocBuilder.FindNodeConfiguration(NodeID);
			if (!Config.IsValid())
			{
				return;
			}
			TSharedPtr<const Metasound::IOperatorData> OperatorData = Config.Get().GetOperatorData();
			// Safe downcast because this operator data type is associated with the node config type associated with this customization type
			const TSharedPtr<const Metasound::Experimental::FWidgetExampleOperatorData> WidgetOperatorData = StaticCastSharedPtr<const Metasound::Experimental::FWidgetExampleOperatorData>(OperatorData);
			// Const cast because operator data getter is currently only const
			TSharedPtr<Metasound::Experimental::FWidgetExampleOperatorData> MutableWidgetOperatorData = ConstCastSharedPtr<Metasound::Experimental::FWidgetExampleOperatorData>(WidgetOperatorData);
			MyFloatPropertyHandle->GetValue(MutableWidgetOperatorData->MyFloat);
		}
	}
}

#undef LOCTEXT_NAMESPACE
