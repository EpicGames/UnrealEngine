// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinitions/AssetDefinition_CatSoundWaveContainer.h"
#include "IMetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorModule.h"
#include "MetasoundExampleNodeConfiguration.h"
#include "MetasoundExampleNodeDetailsCustomization.h"
#include "MetasoundMappingFunctionNode.h"
#include "MetasoundMappingFunctionDetailsCustomization.h"
#include "MetasoundGranulatorNode.h"
#include "MetasoundGranularNodeDetailsCustomization.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalEditor"

class FMetasoundExperimentalEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface API
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface API

private:
	void RegisterMenus();
};

void FMetasoundExperimentalEditorModule::StartupModule()
{
	using namespace Metasound::Editor;
	IMetasoundEditorModule& MetasoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	MetasoundEditorModule.RegisterCustomNodeConfigurationDetailsCustomization(
		FMetaSoundWidgetExampleNodeConfiguration::StaticStruct()->GetFName(),
		[](TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
		{
			return MakeShareable(new FExampleWidgetNodeConfigurationCustomization(InStructProperty, InNode));
		});
	MetasoundEditorModule.RegisterCustomNodeConfigurationDetailsCustomization(
		FMetaSoundMappingFunctionNodeConfiguration::StaticStruct()->GetFName(),
		[](TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
		{
			return MakeShareable(new FMappingFunctionNodeConfigurationCustomization(InStructProperty, InNode));
		});
	MetasoundEditorModule.RegisterCustomNodeConfigurationDetailsCustomization(
		FMetaSoundGranulatorNodeConfiguration::StaticStruct()->GetFName(),
		[](TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
		{
			return MakeShareable(new FGranularNodeConfigurationCustomization(InStructProperty, InNode));
		});
	TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = MetasoundEditorModule.GetGraphPanelPinFactory();
	PinFactory->RegisterPin("Enum:CurveFunction", FGraphPinParams { .PinCategory = "Int32" });

	PinFactory->RegisterPin("CatExperimental::SoundWaveContainerAsset");

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FMetasoundExperimentalEditorModule::RegisterMenus));
}

void FMetasoundExperimentalEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped MenuOwner(this);
	FCatSoundWaveContainerExtension::RegisterMenus();
}

void FMetasoundExperimentalEditorModule::ShutdownModule()
{
	using namespace Metasound::Editor;
	if (FModuleManager::Get().IsModuleLoaded(IMetasoundEditorModule::ModuleName))
	{
		IMetasoundEditorModule& MetasoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
		MetasoundEditorModule.UnregisterCustomNodeConfigurationDetailsCustomization(FMetaSoundWidgetExampleNodeConfiguration::StaticStruct()->GetFName());
		MetasoundEditorModule.UnregisterCustomNodeConfigurationDetailsCustomization(FMetaSoundMappingFunctionNodeConfiguration::StaticStruct()->GetFName());
		MetasoundEditorModule.UnregisterCustomNodeConfigurationDetailsCustomization(FMetaSoundGranulatorNodeConfiguration::StaticStruct()->GetFName());
		TSharedRef<IMetaSoundGraphPanelPinFactory> PinFactory = MetasoundEditorModule.GetGraphPanelPinFactory();
		PinFactory->UnregisterPin("Enum:CurveFunction");
		PinFactory->UnregisterPin("CatExperimental::SoundWaveContainerAsset");
	}

	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMetasoundExperimentalEditorModule, MetasoundExperimentalEditor)