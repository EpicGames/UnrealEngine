// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGranularNodeDetailsCustomization.h" 

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundGranulatorNode.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalEditor"

namespace MetasoundGranularNodeDetailsCustomizationPrivate
{
	const FString StructIdentifierWithDelimiters =  TEXT(".Struct.");
}

FGranularNodeConfigurationCustomization::FGranularNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
	: Metasound::Editor::FMetaSoundNodeConfigurationCustomization(InStructProperty, InNode)
{
	if (InStructProperty && InStructProperty->IsValidHandle())
	{
		StructPropertyPath = InStructProperty->GeneratePathToProperty();
	}
}

void FGranularNodeConfigurationCustomization::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	TSharedPtr<IPropertyHandle> ChildHandle = ChildRow.GetPropertyHandle();
	if (!ChildHandle || !ChildHandle->IsValidHandle())
	{
		return;
	}

	const FString PropertyPath = ChildHandle->GeneratePathToProperty();

	using namespace MetasoundGranularNodeDetailsCustomizationPrivate;
	if (PropertyPath == StructPropertyPath + StructIdentifierWithDelimiters + GET_MEMBER_NAME_CHECKED(FMetaSoundGranulatorNodeConfiguration, GranularEnvelope).ToString())
 	{
		EnvelopeTypePropertyHandle = ChildHandle;
	}

	// Add custom onvalue changed
	TDelegate<void(const FPropertyChangedEvent&)> OnValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FGranularNodeConfigurationCustomization::OnChildPropertyChanged);
	ChildHandle->SetOnPropertyValueChangedWithData(OnValueChangedDelegate);

	// Add base class on value changed
	Metasound::Editor::FMetaSoundNodeConfigurationCustomization::OnChildRowAdded(ChildRow);
}

void FGranularNodeConfigurationCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMetaSoundGranulatorNodeConfiguration, GranularEnvelope))
	{
		if (GraphNode.IsValid() && EnvelopeTypePropertyHandle.IsValid() && EnvelopeTypePropertyHandle->IsValidHandle())
		{
			FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
			const FGuid& NodeID = GraphNode->GetNodeID();

			// Update the operator data value from the configuration property handle value
			// Node configuration operator data API is experimental
			// so this code will be made cleaner in the future 
			TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Config = DocBuilder.FindNodeConfiguration(NodeID);
			TSharedPtr<const Metasound::IOperatorData> OperatorData = Config.Get().GetOperatorData();
			const TSharedPtr<const Metasound::GranulatorPrivate::FGranulatorOperatorData> GranularOperatorData = StaticCastSharedPtr<const Metasound::GranulatorPrivate::FGranulatorOperatorData>(OperatorData);
			TSharedPtr<Metasound::GranulatorPrivate::FGranulatorOperatorData> MutableOperatorData = ConstCastSharedPtr<Metasound::GranulatorPrivate::FGranulatorOperatorData>(GranularOperatorData);

			TArray<void*> RawData;
			EnvelopeTypePropertyHandle->AccessRawData(RawData);
			if (RawData.Num() > 0)
			{
				EGranularEnvelope* GranularEnvelope = static_cast<EGranularEnvelope*>(RawData[0]);
				MutableOperatorData->GranularEnvelope = *GranularEnvelope;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
