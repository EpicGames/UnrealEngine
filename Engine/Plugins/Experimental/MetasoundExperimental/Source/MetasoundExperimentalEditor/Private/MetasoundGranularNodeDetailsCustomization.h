// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundNodeConfigurationCustomization.h"

class FGranularNodeConfigurationCustomization : public Metasound::Editor::FMetaSoundNodeConfigurationCustomization
{
public:
	FGranularNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode);

	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;

private:
	void OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	FString StructPropertyPath;
	TSharedPtr<IPropertyHandle> EnvelopeTypePropertyHandle;
};
