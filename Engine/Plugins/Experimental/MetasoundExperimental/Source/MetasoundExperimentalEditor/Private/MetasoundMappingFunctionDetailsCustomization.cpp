// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundMappingFunctionDetailsCustomization.h" 

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "MetasoundEditorGraphNode.h"
#include "MetasoundBuilderBase.h"
#include "MetasoundMappingFunctionNode.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SBox.h"
#include "SCurveEditor.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalEditor"

FMappingFunctionNodeConfigurationCustomization::FMappingFunctionNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode)
	: Metasound::Editor::FMetaSoundNodeConfigurationCustomization(InStructProperty, InNode)
{
	if (InStructProperty && InStructProperty->IsValidHandle())
	{
		StructPropertyPath = InStructProperty->GeneratePathToProperty();
	}
}

TArray<FRichCurveEditInfo> FMappingFunctionNodeConfigurationCustomization::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	if (RuntimeCurve)
	{
		// Provide the FRichCurve for editing
		Curves.Add(FRichCurveEditInfo(RuntimeCurve->GetRichCurve()));
	}
	return Curves;
}

TArray<FRichCurveEditInfoConst> FMappingFunctionNodeConfigurationCustomization::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	if (RuntimeCurve)
	{
		// Provide the FRichCurve for editing
		Curves.Add(FRichCurveEditInfoConst(RuntimeCurve->GetRichCurve()));
	}
	return Curves;
}

void FMappingFunctionNodeConfigurationCustomization::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const 
{
	if (RuntimeCurve)
	{
		Curves.Add(FRichCurveEditInfoConst(RuntimeCurve->GetRichCurveConst()));
	}
}


bool FMappingFunctionNodeConfigurationCustomization::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	// Validate that the curve being edited is the one we expect
	return RuntimeCurve && (CurveInfo.CurveToEdit == RuntimeCurve->GetRichCurve());
}

void FMappingFunctionNodeConfigurationCustomization::MakeTransactional()
{
	// Mark owners as transactional to support undo/redo
	for (UObject* Owner : OwnerObjects)
	{
		if (Owner)
		{
			Owner->SetFlags(RF_Transactional);
		}
	}
}

void FMappingFunctionNodeConfigurationCustomization::ModifyOwner()
{
	// Called at the start of a curve edit (begin transaction)
	for (UObject* Owner : OwnerObjects)
	{
		if (Owner)
		{
			Owner->Modify();
		}
	}
}

void FMappingFunctionNodeConfigurationCustomization::ModifyOwnerChange()
{
	// Called during interactive changes (e.g., while dragging a key)
	for (UObject* Owner : OwnerObjects)
	{
		if (Owner)
		{
			Owner->Modify();
		}
	}
}

void FMappingFunctionNodeConfigurationCustomization::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	// Enforce clamping and endpoint constraints whenever the curve is edited
	FRichCurve* RichCurve = RuntimeCurve ? RuntimeCurve->GetRichCurve() : nullptr;
	if (!RichCurve)
	{
		return;
	}

	bool bCurveModified = false;

	// Clamp all keys within [0,1] for both X (Time) and Y (Value)
	for (FRichCurveKey& Key : RichCurve->Keys)
	{
		if (Key.Time < 0.0f) { Key.Time = 0.0f;  bCurveModified = true; }
		if (Key.Time > 1.0f) { Key.Time = 1.0f;  bCurveModified = true; }
		if (Key.Value < 0.0f) { Key.Value = 0.0f; bCurveModified = true; }
		if (Key.Value > 1.0f) { Key.Value = 1.0f; bCurveModified = true; }
	}

	// Ensure a key exists at X = 0.0 (lock first key's X position to 0)
	if (RichCurve->Keys.Num() == 0 || RichCurve->Keys[0].Time > 0.0f)
	{
		float NewValue = (RichCurve->Keys.Num() > 0) ? RichCurve->Eval(0.0f) : 0.0f;
		RichCurve->AddKey(0.0f, NewValue);
		RichCurve->Keys[0].InterpMode = RCIM_Linear;
		bCurveModified = true;
	}
	// Ensure a key exists at X = 1.0 (lock last key's X position to 1)
	if (RichCurve->Keys.Num() == 0 || RichCurve->Keys.Last().Time < 1.0f)
	{
		float NewValue = (RichCurve->Keys.Num() > 0) ? RichCurve->Eval(1.0f) : 1.0f;
		RichCurve->AddKey(1.0f, NewValue);
		RichCurve->Keys.Last().InterpMode = RCIM_Linear;
		bCurveModified = true;
	}

	// Snap the first and last keys exactly to 0.0 and 1.0 on X (in case they were moved)
	if (RichCurve->Keys.Num() >= 2)
	{
		FRichCurveKey& FirstKey = RichCurve->Keys[0];
		FRichCurveKey& LastKey = RichCurve->Keys.Last();
		if (!FMath::IsNearlyZero(FirstKey.Time))
		{
			FirstKey.Time = 0.0f;
			bCurveModified = true;
		}
		if (!FMath::IsNearlyEqual(LastKey.Time, 1.0f))
		{
			LastKey.Time = 1.0f;
			bCurveModified = true;
		}
	}

	if (bCurveModified)
	{
		// Notify the engine that the property value has changed, to support undo/redo and UI refresh
		FProperty* CurveProperty = CurvePropertyHandle.IsValid() ? CurvePropertyHandle->GetProperty() : nullptr;
		if (CurveProperty)
		{
			for (UObject* Owner : OwnerObjects)
			{
				if (Owner)
				{
					Owner->PreEditChange(CurveProperty);
				}
			}
			FPropertyChangedEvent ChangeEvent(CurveProperty, EPropertyChangeType::ValueSet);
			for (UObject* Owner : OwnerObjects)
			{
				if (Owner)
				{
					Owner->PostEditChangeProperty(ChangeEvent);
				}
			}
		}
	}

	UpdateMappingFunctionData();

	return;
}

TArray<const UObject*> FMappingFunctionNodeConfigurationCustomization::GetOwners() const
{
	TArray<const UObject*> Result;
	for (UObject* Owner : OwnerObjects)
	{
		if (Owner)
		{
			Result.Add(Owner);
		}
	}
	return Result;
}

void FMappingFunctionNodeConfigurationCustomization::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	TSharedPtr<IPropertyHandle> ChildHandle = ChildRow.GetPropertyHandle();
	if (!ChildHandle || !ChildHandle->IsValidHandle())
	{
		return;
	}

	const FString PropertyPath = ChildHandle->GeneratePathToProperty();

	if (PropertyPath == StructPropertyPath + TEXT(".Struct.") + GET_MEMBER_NAME_CHECKED(FMetaSoundMappingFunctionNodeConfiguration, MappingFunction).ToString())
	{
		CurvePropertyHandle = ChildHandle;
		RuntimeCurve = nullptr;
		OwnerObjects.Empty();

		// Access the raw struct data and cast to a curve
		TArray<void*> RawData;
		CurvePropertyHandle->AccessRawData(RawData);
		if (RawData.Num() > 0)
		{
			RuntimeCurve = static_cast<FRuntimeFloatCurve*>(RawData[0]);
		}
		if (!RuntimeCurve)
		{
			return;
		}

		// Get outer owning UObject(s)
		CurvePropertyHandle->GetOuterObjects(OwnerObjects);

		// Ensure default keys at 0 and 1 with linear mapping if the curve is empty
		FRichCurve* RichCurve = RuntimeCurve->GetRichCurve();
		if (RichCurve->GetNumKeys() == 0)
		{
			RichCurve->AddKey(0.0f, 0.0f);
			RichCurve->AddKey(1.0f, 1.0f);

			// Set interpolation to linear for a straight line between (0,0) and (1,1)
			RichCurve->Keys[0].InterpMode = RCIM_Linear;
			RichCurve->Keys.Last().InterpMode = RCIM_Linear;
		}

		CurveEditorWidget = SNew(SCurveEditor)
			.ViewMinInput(0.0f)
			.ViewMaxInput(1.0f)
			.ViewMinOutput(0.0f)
			.ViewMaxOutput(1.0f)
			.ZoomToFitHorizontal(false)
			.ZoomToFitVertical(false)
			.ShowInputGridNumbers(false)
			.ShowOutputGridNumbers(false)
			.AllowZoomOutput(false)
			.TimelineLength(1.0f)
			.HideUI(true)
			.ShowZoomButtons(false)
			.DesiredSize(FVector2D(300,200))
			.ShowCurveSelector(false);

		CurveEditorWidget->SetCurveOwner(this);

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow Row;

		ChildRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
		ChildRow.CustomWidget(true)
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			.MinDesiredWidth(200)
			.MaxDesiredWidth(400)
			[
				CurveEditorWidget.ToSharedRef()
			];
	}
	else if (PropertyPath == StructPropertyPath + TEXT(".Struct.") + GET_MEMBER_NAME_CHECKED(FMetaSoundMappingFunctionNodeConfiguration, bWrapInputs).ToString())
	{
		bWrapInputsPropertyHandle = ChildHandle;
	}

	// Add custom onvalue changed
	TDelegate<void(const FPropertyChangedEvent&)> OnValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMappingFunctionNodeConfigurationCustomization::OnChildPropertyChanged);
	ChildHandle->SetOnPropertyValueChangedWithData(OnValueChangedDelegate);

	// Add base class on value changed
	Metasound::Editor::FMetaSoundNodeConfigurationCustomization::OnChildRowAdded(ChildRow);

	// Make sure we update the mapping function data since we added some default keys
	UpdateMappingFunctionData();
}

void FMappingFunctionNodeConfigurationCustomization::UpdateMappingFunctionData()
{
	if (GraphNode.IsValid())
	{
		FMetaSoundFrontendDocumentBuilder& DocBuilder = GraphNode->GetBuilderChecked().GetBuilder();
		const FGuid& NodeID = GraphNode->GetNodeID();

		TInstancedStruct<FMetaSoundFrontendNodeConfiguration> Config = DocBuilder.FindNodeConfiguration(NodeID);
		TSharedPtr<const Metasound::IOperatorData> OperatorData = Config.Get().GetOperatorData();
		const TSharedPtr<const Metasound::Experimental::FMappingFunctionNodeOperatorData> MappingFunctionOperatorData = StaticCastSharedPtr<const Metasound::Experimental::FMappingFunctionNodeOperatorData>(OperatorData);
		TSharedPtr<Metasound::Experimental::FMappingFunctionNodeOperatorData> MutableMappingFunctionOperatorData = ConstCastSharedPtr<Metasound::Experimental::FMappingFunctionNodeOperatorData>(MappingFunctionOperatorData);

		if (CurvePropertyHandle.IsValid() &&
			CurvePropertyHandle->IsValidHandle())
		{
			TArray<void*> RawData;
			CurvePropertyHandle->AccessRawData(RawData);
			if (RawData.Num() > 0)
			{
				FRuntimeFloatCurve* Curve = static_cast<FRuntimeFloatCurve*>(RawData[0]);
				MutableMappingFunctionOperatorData->MappingFunction = *Curve;
			}
		}
		else if (bWrapInputsPropertyHandle.IsValid() &&
				 bWrapInputsPropertyHandle->IsValidHandle())
		{
			bWrapInputsPropertyHandle->GetValue(MutableMappingFunctionOperatorData->bWrapInputs);
		}
	}
}


void FMappingFunctionNodeConfigurationCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UpdateMappingFunctionData();

}

#undef LOCTEXT_NAMESPACE
