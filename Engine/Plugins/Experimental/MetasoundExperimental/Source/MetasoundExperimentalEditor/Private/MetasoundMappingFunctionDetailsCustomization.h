// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetasoundNodeConfigurationCustomization.h"
#include "SCurveEditor.h"
#include "Curves/CurveFloat.h"

class FMappingFunctionNodeConfigurationCustomization : public Metasound::Editor::FMetaSoundNodeConfigurationCustomization, public FCurveOwnerInterface
{
public:
	FMappingFunctionNodeConfigurationCustomization(TSharedPtr<IPropertyHandle> InStructProperty, TWeakObjectPtr<UMetasoundEditorGraphNode> InNode);

	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;

	// FCurveOwnerInterface interface
	virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual bool HasRichCurves() const override { return true; }
	virtual bool IsLinearColorCurve() const override { return false; }
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;
	virtual void MakeTransactional() override;
	virtual void ModifyOwner() override;
	virtual void ModifyOwnerChange() override;
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual TArray<const UObject*> GetOwners() const override;

private:
	void OnChildPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	void UpdateMappingFunctionData();

	FString StructPropertyPath;
	TSharedPtr<IPropertyHandle> CurvePropertyHandle;
	TSharedPtr<IPropertyHandle> bWrapInputsPropertyHandle;

	FRuntimeFloatCurve* RuntimeCurve = nullptr;
	TArray<UObject*> OwnerObjects;
	TSharedPtr<SCurveEditor> CurveEditorWidget;
};
