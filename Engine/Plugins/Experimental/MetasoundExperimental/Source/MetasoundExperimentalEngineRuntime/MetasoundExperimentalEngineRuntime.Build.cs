// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MetasoundExperimentalEngineRuntime : ModuleRules
{
	public MetasoundExperimentalEngineRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		bValidateExperimentalApi = false;
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MetasoundGraphCore",
				"MetasoundFrontend",
				"AudioExperimentalRuntime",
				"MetasoundExperimentalRuntime"
			}
		);
		
		// When the Editor builds all plugins, it will build with engine. In that case add a dependency on CoreUObject.
		// If we don't do this we will get linker errors for the Engine expecting to find stuff defined here.
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("CoreUObject");
		}
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"MetasoundGraphCore",
				"MetasoundFrontend",
				"MetasoundEngine",
				"MetasoundStandardNodes",
				"SignalProcessing",
				"MetasoundExperimentalRuntime",
				"AudioChannelAgnosticCore",
				"AudioExtensions",
			}
		);

		PrivateDefinitions.AddRange(
			new string[]
			{
				"METASOUND_PLUGIN=MetasoundExperimental",
				"METASOUND_MODULE=MetasoundExperimentalEngineRuntime"
			}
		);
	}
}
