// Copyright Epic Games, Inc. All Rights Reserved.

#include "CatSoundWaveContainer.h"

#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Sound/SoundWave.h"
#include "UObject/UnrealNames.h"

DEFINE_LOG_CATEGORY_STATIC(LogCatSoundWaveContainer, Log, All);

#if WITH_EDITORONLY_DATA

void UCatSoundWaveContainer::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	if (Proxy.IsValid())
	{
		FCatSoundWaveContainerData NewData(Proxy->GetData().InitParams, *this);
		Proxy = Proxy->New(MoveTemp(NewData));
	}
}

#endif // WITH_EDITORONLY_DATA

TSharedPtr<Audio::IProxyData> UCatSoundWaveContainer::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!Proxy)
	{
		FCatSoundWaveContainerData Data(InitParams, *this);
		Proxy = FCatSoundWaveContainerProxy::Create(MoveTemp(Data));
	}
	return Proxy;
}

void UCatSoundWaveContainer::RebuildProxy()
{
	if (Proxy.IsValid())
	{
		FCatSoundWaveContainerData NewData(Proxy->GetData().InitParams, *this);
		Proxy = Proxy->New(MoveTemp(NewData));
	}
	else
	{
		Audio::FProxyDataInitParams InitParams;
		InitParams.NameOfFeatureRequestingProxy = TEXT("CatSoundWaveContainer");
		FCatSoundWaveContainerData Data(InitParams, *this);
		Proxy = FCatSoundWaveContainerProxy::Create(MoveTemp(Data));
	}
}

FCatSoundWaveContainerData::FCatSoundWaveContainerData(const Audio::FProxyDataInitParams& InInitParams)
	: InitParams(InInitParams)
{
}

FCatSoundWaveContainerData::FCatSoundWaveContainerData(const Audio::FProxyDataInitParams& InInitParams, const UCatSoundWaveContainer& SoundWaveContainer)
	: InitParams(InInitParams)
{
	Update(SoundWaveContainer);
}

void FCatSoundWaveContainerData::Update(const UCatSoundWaveContainer& SoundWaveContainer)
{
	Type = SoundWaveContainer.Type;
	Entries.SetNum(SoundWaveContainer.Entries.Num());
	for (int32 Idx = 0; Idx < SoundWaveContainer.Entries.Num(); ++Idx)
	{
		Entries[Idx] = FEntry(InitParams, SoundWaveContainer.Entries[Idx]);
		Entries[Idx].Weight = SoundWaveContainer.Entries[Idx].Weight;

		if (!Entries[Idx].SoundWave.IsValid())
		{
			UE_LOGFMT(LogCatSoundWaveContainer, Warning,
				"Invalid or null asset in CAT SoundWaveContainer at Entry[{0}] ({1})",
				Idx, SoundWaveContainer.GetFName());
		}
	}
}

TArray<FSoundWaveProxyPtr> FCatSoundWaveContainerData::GetContainedWaveProxies() const
{
	TArray<FSoundWaveProxyPtr> Result;
	Result.Reserve(Entries.Num());
	for (const FEntry& Entry : Entries)
	{
		if (Entry.SoundWave.IsValid())
		{
			Result.Add(Entry.SoundWave);
		}
	}
	return Result;
}

FCatSoundWaveContainerData::FEntry::FEntry(const Audio::FProxyDataInitParams& InInitParams, const FCatSoundWaveContainerEntry& Entry)
{
	if (!Entry.SoundWave)
	{
		return;
	}

	TSharedPtr<Audio::IProxyData> ProxyData = Entry.SoundWave->CreateProxyData(InInitParams);
	if (ProxyData.IsValid() && ensure(ProxyData->CheckTypeCast<FSoundWaveProxy>()))
	{
		SoundWave = StaticCastSharedPtr<FSoundWaveProxy>(ProxyData);
	}

	if (!ensure(SoundWave))
	{
		UE_LOGFMT(LogCatSoundWaveContainer, Error,
			"Asset in CAT SoundWaveContainer is not a SoundWave proxy: {0}",
			Entry.SoundWave->GetFName());
	}
}
