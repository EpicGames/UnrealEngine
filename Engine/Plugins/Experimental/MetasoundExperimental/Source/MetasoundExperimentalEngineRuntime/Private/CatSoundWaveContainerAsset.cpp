// Copyright Epic Games, Inc. All Rights Reserved.

#include "CatSoundWaveContainerAsset.h"

#include "CatSoundWaveContainer.h"

REGISTER_METASOUND_DATATYPE(
	MetasoundCatExperimental::FSoundWaveContainerAsset,
	"CatExperimental::SoundWaveContainerAsset",
	Metasound::ELiteralType::UObjectProxy,
	UCatSoundWaveContainer);

namespace MetasoundCatExperimental
{
	FSoundWaveContainerAsset::FSoundWaveContainerAsset(const TSharedPtr<Audio::IProxyData>& InData)
	{
		if (InData.IsValid() && InData->CheckTypeCast<FCatSoundWaveContainerProxy>())
		{
			Proxy = StaticCastSharedPtr<FCatSoundWaveContainerProxy>(InData);
		}
	}

	TSharedPtr<const FCatSoundWaveContainerProxy> FSoundWaveContainerAsset::GetLatest() const
	{
		if (Proxy.IsValid() && !Proxy->IsLatest())
		{
			Proxy = Proxy->GetLatest();
		}
		return Proxy;
	}
}
