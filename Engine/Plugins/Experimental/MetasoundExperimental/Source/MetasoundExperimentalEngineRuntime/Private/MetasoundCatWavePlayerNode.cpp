// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatWavePlayerNode.h"

#include "Algo/MaxElement.h"
#include "ChannelAgnostic/ChannelAgnosticTranscoding.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/MultichannelBuffer.h"
#include "DSP/MultiMono.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Math/RandomStream.h"
#include "Math/UnrealMathUtility.h"
#include "MetasoundBuildError.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundDataFactory.h"
#include "MetasoundDataReference.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundOperatorSettings.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProxyPlayer.h"
#include "CatSoundWaveContainer.h"
#include "CatSoundWaveContainerAsset.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalNodes_CatWavePlayer"

DEFINE_LOG_CATEGORY_STATIC(LogCatWavePlayer, Log, All);

namespace Metasound::CatWavePlayer
{
	// All possible pins. Which ones are exposed depends upon the node configuration.
	METASOUND_PARAM(InputPlay,               "Play",                         "Trigger to start playing a new wave voice.");
	METASOUND_PARAM(InputStopAll,            "Stop All",                     "Stops all currently playing voices.");
	METASOUND_PARAM(InputSoundWaveContainer, "Sound Wave Container",         "Source container of waves for playback.");
	METASOUND_PARAM(InputPitchShift,         "Pitch Shift",                  "Pitch shift in semitones. Accumulated with container and entry-level pitch settings.");
	METASOUND_PARAM(InputIndex,              "Index",                        "Index of the entry to play from the container (Index mode only).");
	METASOUND_PARAM(InputSequenceStartIndex, "Sequence Start Index",         "Starting index into the sequence (negative wraps from end).");
	METASOUND_PARAM(InputNumSequenceLoops,   "Num Sequence Loops",           "Loop count. -1 = infinite, 0 = play nothing (explicit opt-out), >0 = play that many passes.");
	METASOUND_PARAM(InputResetSequence,      "Reset Sequence",               "Trigger to reset sequence position to the start without stopping active voices.");
	METASOUND_PARAM(InputSeed,               "Seed",                         "Random seed. -1 = time-based.");
	METASOUND_PARAM(InputAvoidRepeatLast,    "Avoid Repeating Last",         "Window of recently played entries to exclude during random selection.");

	METASOUND_PARAM(OutputAudio,                     "Audio Out",                   "Mixed polyphonic CAT output.");
	METASOUND_PARAM(OutputOnVoicesStarted,           "On Voices Started",           "Fires when a Play triggers and no voices were previously active.");
	METASOUND_PARAM(OutputOnVoicesFinished,          "On Voices Finished",          "Fires when all voices finish (paired with On Voices Started).");
	METASOUND_PARAM(OutputOnActiveVoiceCountChanged, "On Active Voice Count Changed", "Fires when the active voice count changes.");
	METASOUND_PARAM(OutputActiveVoiceCount,          "Active Voice Count",          "Current number of active voices.");
	METASOUND_PARAM(OutputOnSequenceFirstSound,      "On Sequence First Sound Played", "Fires when the first sound of the first loop is played.");
	METASOUND_PARAM(OutputOnSequenceLastSound,       "On Sequence Last Sound Played",  "Fires when the last sound of the last loop is played.");
	METASOUND_PARAM(OutputOnSequenceLooped,          "On Sequence Looped",          "Fires when the sequence wraps to the start.");
	METASOUND_PARAM(OutputOnSequenceFinished,        "On Sequence Finished",        "Fires when the sequence finishes.");

	// Not all possible formats are currently supported by this waveplayer because
	// of the ability for the AudioMixer to support the format. When the AudioMixer
	// supports CAT internally, then these limits can be removed. 
	static bool IsSupportedCatFormat(FName InFormat)
	{
		static const FName SupportedFormats[] = {
			TEXT("Cat:Mono1Dot0"),
			TEXT("Cat:Stereo2Dot0"),
			TEXT("Cat:Quad4Dot0Side"),
			TEXT("Cat:Surround5Dot1"),
			TEXT("Cat:Surround7Dot1"),
		};
		for (const FName& Name : SupportedFormats)
		{
			if (Name == InFormat)
			{
				return true;
			}
		}
		return false;
	}

	// A bit of a temporary hack for finding formats via channel counts. 
	static FName NumChannelsToCatFormat(int32 NumChannels)
	{
		// "Largest format across inputs" clamping rule from main spec §4 Confirmed #7.
		// Non-MVP channel counts round up to the next MVP format that fits.
		if (NumChannels <= 1) return TEXT("Cat:Mono1Dot0");
		if (NumChannels <= 2) return TEXT("Cat:Stereo2Dot0");
		if (NumChannels <= 4) return TEXT("Cat:Quad4Dot0Side");
		if (NumChannels <= 6) return TEXT("Cat:Surround5Dot1");
		return TEXT("Cat:Surround7Dot1");
	}

	// ---------------------------------------------------------------------
	// Operator data — opaque configuration passed through INode::GetOperatorData().
	// ---------------------------------------------------------------------
	class FCatWavePlayerOperatorData final : public TOperatorData<FCatWavePlayerOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;

		FCatWavePlayerOperatorData() = default;

		int32                         MaxVoices    = 2;
		ECatWavePlayerFormatChooser   Format       = ECatWavePlayerFormatChooser::SourceAuto;
		FName                         CustomFormat = TEXT("Cat:Stereo2Dot0");
		ECatWavePlayerPlaybackType    PlaybackType = ECatWavePlayerPlaybackType::Index;
		ECatWavePlayerPlaybackMode    PlaybackMode = ECatWavePlayerPlaybackMode::Standard;
	};
	const FLazyName FCatWavePlayerOperatorData::OperatorDataTypeName = TEXT("FCatWavePlayerOperatorData");

	// ---------------------------------------------------------------------
	// Vertex-interface construction. Shape depends on configuration.
	// ---------------------------------------------------------------------
	static FVertexInterface GetVertexInterface(const FCatWavePlayerOperatorData& Config)
	{
		FInputVertexInterface Inputs{
			TInputDataVertex<FTrigger>                                       (METASOUND_GET_PARAM_NAME_AND_METADATA(InputPlay)),
			TInputDataVertex<FTrigger>                                       (METASOUND_GET_PARAM_NAME_AND_METADATA(InputStopAll)),
			TInputDataVertex<MetasoundCatExperimental::FSoundWaveContainerAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSoundWaveContainer)),
			TInputDataVertex<float>                                          (METASOUND_GET_PARAM_NAME_AND_METADATA(InputPitchShift), 0.0f),
		};

		if (Config.PlaybackMode == ECatWavePlayerPlaybackMode::Standard && Config.PlaybackType == ECatWavePlayerPlaybackType::Index)
		{
			Inputs.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputIndex), 0));
		}

		if (Config.PlaybackType == ECatWavePlayerPlaybackType::Sequence)
		{
			Inputs.Add(TInputDataVertex<int32>    (METASOUND_GET_PARAM_NAME_AND_METADATA(InputSequenceStartIndex), 0));
			Inputs.Add(TInputDataVertex<int32>    (METASOUND_GET_PARAM_NAME_AND_METADATA(InputNumSequenceLoops),   -1));
			Inputs.Add(TInputDataVertex<FTrigger> (METASOUND_GET_PARAM_NAME_AND_METADATA(InputResetSequence)));
		}

		if (Config.PlaybackMode == ECatWavePlayerPlaybackMode::Random)
		{
			Inputs.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSeed),            -1));
			Inputs.Add(TInputDataVertex<int32>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAvoidRepeatLast),  0));
		}

		// Declare Audio Out with the polymorphic base CAT type ("Cat"). The concrete format
		// (e.g. "Cat:Stereo2Dot0") is resolved at CreateOperator time via ResolveOutputFormat
		// and stamped on the FChannelAgnosticType data reference. Downstream bind via
		// Metasound::IsCastable walks the poly-type hierarchy, so any concrete CAT format
		// binds successfully. Pinning the vertex to a concrete format here produced a
		// divergence bug where OverrideDefaultInterface-time and CreateOperator-time
		// resolutions disagreed (e.g. Stereo at the vertex, Mono in the data ref), tripping
		// the bind-time checkf in MetasoundVertexData.h.
		//
		// This is something we may want to revisit for scenarios when the CAT format
		// is hardcoded in the configuration. 
		FOutputVertexInterface Outputs{
			FOutputDataVertex(
				METASOUND_GET_PARAM_NAME(OutputAudio),
				GetMetasoundDataTypeName<FChannelAgnosticType>(),
				METASOUND_GET_PARAM_METADATA(OutputAudio),
				EVertexAccessType::Reference),
			TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnVoicesStarted)),
			TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnVoicesFinished)),
			TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(OutputOnActiveVoiceCountChanged)),
			TOutputDataVertex<int32>   (METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(OutputActiveVoiceCount)),
		};

		if (Config.PlaybackType == ECatWavePlayerPlaybackType::Sequence)
		{
			Outputs.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSequenceFirstSound)));
			Outputs.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSequenceLastSound)));
			Outputs.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSequenceLooped)));
			Outputs.Add(TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnSequenceFinished)));
		}

		return FVertexInterface{ MoveTemp(Inputs), MoveTemp(Outputs) };
	}

	// ---------------------------------------------------------------------
	// FInternalVoice — self-contained polyphonic voice. 
	// ---------------------------------------------------------------------
	struct FInternalVoice
	{
		TUniquePtr<FSoundWaveProxyPlayer> ProxyPlayer;

		/** Planar (per-channel) output of the proxy player's native resolution. */
		Audio::FMultichannelBuffer MultiChannelBuffers;
		Audio::FMultichannelBufferView MultiChannelView;

		float PitchShiftSemitones = 0.0f;
		bool bIsPlaying = false;
		bool bIsFinished = true;
		int32 SourceNumChannels = 0;
		uint32 ActivationOrder = 0;

		void Initialize(float InSampleRate, int32 InNumFramesPerBlock)
		{
			// Player output must be rebuilt per-wave because channel counts change; settings
			// below are provisional (OutputNumChannels is clamped to 1 for the placeholder
			// player that is never rendered before StartPlaying).
			FSoundWaveProxyPlayer::FSettings Settings(InSampleRate, 1);
			ProxyPlayer = FSoundWaveProxyPlayer::Create(Settings);
			check(ProxyPlayer.IsValid());

			MultiChannelBuffers.Reset();
			MultiChannelView.Reset();
			MultiChannelView.SetNum(0);
			SourceNumChannels = 0;
			bIsPlaying = false;
			bIsFinished = true;
			PitchShiftSemitones = 0.0f;
			ActivationOrder = 0;
		}

		void StartPlaying(const TSharedPtr<FSoundWaveProxy>& WaveProxy, float InPitchShiftSemitones, float InSampleRate, int32 InNumFramesPerBlock, uint32 InActivationOrder)
		{
			check(ProxyPlayer.IsValid());
			if (!WaveProxy.IsValid())
			{
				bIsPlaying = false;
				bIsFinished = true;
				return;
			}

			const int32 WaveNumChannels = FMath::Max<int32>(1, WaveProxy->GetSoundWaveDataRef()->GetNumChannels());
			SourceNumChannels = WaveNumChannels;

			// (Re)configure the player at the wave's native channel count. Every active
			// voice renders its source into its source channel count; transcode happens
			// when we mix into the CAT output.
			FSoundWaveProxyPlayer::FSettings Settings(InSampleRate, FMath::Clamp(WaveNumChannels, 1, 8));
			ProxyPlayer->Reset(Settings);
			const bool bWavOk = ProxyPlayer->SetSoundWave(WaveProxy->GetSoundWaveDataRef());
			if (!bWavOk)
			{
				bIsPlaying = false;
				bIsFinished = true;
				return;
			}

			// Build per-channel aligned buffers + matching views. Buffers own memory;
			// the FMultichannelBufferView is rebuilt to reference them for passing into
			// RenderMultiChannelAudio.
			MultiChannelBuffers.SetNum(WaveNumChannels);
			MultiChannelView.SetNum(WaveNumChannels);
			for (int32 Ch = 0; Ch < WaveNumChannels; ++Ch)
			{
				MultiChannelBuffers[Ch].SetNumZeroed(InNumFramesPerBlock);
				MultiChannelView[Ch] = MakeArrayView(MultiChannelBuffers[Ch].GetData(), InNumFramesPerBlock);
			}

			ProxyPlayer->SetLoop(false);
			// guard against non-finite pitch. 
			const float SafePitchSemitones = FMath::IsFinite(InPitchShiftSemitones)
				? InPitchShiftSemitones
				: PitchShiftSemitones;
			PitchShiftSemitones = SafePitchSemitones;
			ProxyPlayer->SetSpeed(FMath::Pow(2.0f, PitchShiftSemitones / 12.0f));
			ActivationOrder = InActivationOrder;
			bIsPlaying = true;
			bIsFinished = false;
		}

		void StopPlaying()
		{
			bIsPlaying = false;
			bIsFinished = true;
		}

		void RenderAudio(int32 InNumFramesPerBlock)
		{
			check(bIsPlaying);
			check(ProxyPlayer.IsValid());

			// Ensure views still cover the block — block count can change across resets.
			if (!MultiChannelBuffers.IsEmpty() && MultiChannelBuffers[0].Num() != InNumFramesPerBlock)
			{
				for (int32 Ch = 0; Ch < MultiChannelBuffers.Num(); ++Ch)
				{
					MultiChannelBuffers[Ch].SetNumZeroed(InNumFramesPerBlock);
					MultiChannelView[Ch] = MakeArrayView(MultiChannelBuffers[Ch].GetData(), InNumFramesPerBlock);
				}
			}
			else
			{
				for (Audio::FAlignedFloatBuffer& Buf : MultiChannelBuffers)
				{
					FMemory::Memzero(Buf.GetData(), Buf.Num() * sizeof(float));
				}
			}

			ProxyPlayer->RenderMultiChannelAudio(MultiChannelView);
			if (ProxyPlayer->IsFinished())
			{
				bIsFinished = true;
			}
		}

		/**
		 * Mix this voice's rendered audio into the CAT output, transcoding through the
		 * transcoder keyed by the source's channel count.
		 */
		void MixIntoOutput(Metasound::FChannelAgnosticType& Dst, TMap<int32, Audio::ChannelAgnosticTranscoder::FTranscoder>& Transcoders, Audio::FChannelAgnosticType& TranscodeScratch, int32 InNumFramesPerBlock) const
		{
			check(bIsPlaying);

			if (MultiChannelBuffers.IsEmpty())
			{
				return;
			}

			const int32 NumSrcChannels = MultiChannelBuffers.Num();
			Audio::ChannelAgnosticTranscoder::FTranscoder* Transcoder = Transcoders.Find(NumSrcChannels);
			if (!Transcoder)
			{
				// Either the container introduced a new channel count mid-block, or the
				// construction-time scan missed one. Skip rather than crash; the operator
				// rebuilds the transcoder map on the next proxy-swap.
				return;
			}

			TranscodeScratch.Zero();

			Audio::TStackArrayOfPointers<const float> Src;
			Src.SetNum(NumSrcChannels);
			for (int32 Ch = 0; Ch < NumSrcChannels; ++Ch)
			{
				Src[Ch] = MultiChannelBuffers[Ch].GetData();
			}

			TArrayView<float> ScratchView = TranscodeScratch.GetRawMultiMono();
			Audio::TStackArrayOfPointers<float> DstPtrs = Audio::MakeMultiMonoPointersFromView(ScratchView, InNumFramesPerBlock, TranscodeScratch.NumChannels());
			(*Transcoder)(Src, DstPtrs, InNumFramesPerBlock);

			const int32 NumDstChannels = Dst.NumChannels();
			check(NumDstChannels == TranscodeScratch.NumChannels());
			for (int32 Ch = 0; Ch < NumDstChannels; ++Ch)
			{
				TArrayView<const float> ScratchChannel = TranscodeScratch.GetChannel(Ch);
				TArrayView<float> DstChannel = Dst.GetChannel(Ch);
				Audio::ArrayMixIn(ScratchChannel, DstChannel);
			}
		}
	};

	// ---------------------------------------------------------------------
	// Format resolution. Runs at bind time 
	// ---------------------------------------------------------------------
	static const FName CatWavePlayerFormatNotSupportedErrorType = TEXT("CatWavePlayerFormatNotSupported");

	static FName ResolveOutputFormat(
		const FCatWavePlayerOperatorData& Config,
		const TSharedPtr<const FCatSoundWaveContainerProxy>& ContainerProxy,
		FName InSourceFormatName,
		FBuildResults* OutResults)
	{
		switch (Config.Format)
		{
		case ECatWavePlayerFormatChooser::Custom:
		{
			if (!IsSupportedCatFormat(Config.CustomFormat))
			{
				// Surface as a build error when an FBuildResults sink is provided
				// (CreateOperator path). The BindInputs path passes nullptr — there's
				// no graph-build sink to publish to once we're past operator creation.
				if (OutResults)
				{
					AddBuildError<FBuildErrorBase>(
						OutResults->Errors,
						CatWavePlayerFormatNotSupportedErrorType,
						FText::Format(
							METASOUND_LOCTEXT("CatWavePlayer_FormatNotSupportedError",
								"Audio Wave Player: Custom format '{0}' is not in the supported set (Mono / Stereo / Quad / 5.1 / 7.1). Falling back to Stereo."),
							FText::FromName(Config.CustomFormat)));
				}
				return TEXT("Cat:Stereo2Dot0");
			}
			return Config.CustomFormat;
		}

		case ECatWavePlayerFormatChooser::Auto:
		{
			// Iterate the bound container's entries and clamp the largest channel
			// count to the supported format set. 
			int32 LargestChannelCount = 1;
			if (ContainerProxy.IsValid())
			{
				const FCatSoundWaveContainerData& Data = ContainerProxy->GetData();
				for (const FCatSoundWaveContainerData::FEntry& Entry : Data.Entries)
				{
					if (Entry.SoundWave.IsValid())
					{
						LargestChannelCount = FMath::Max<int32>(LargestChannelCount, Entry.SoundWave->GetSoundWaveDataRef()->GetNumChannels());
					}
				}
			}
			return NumChannelsToCatFormat(LargestChannelCount);
		}

		case ECatWavePlayerFormatChooser::SourceAuto:
		{
			// Inherit from the enclosing UMetaSoundSource. The Source-side resolved
			// format is published via SourceInterface::Environment::SourceFormatName;
			if (InSourceFormatName != NAME_None)
			{
				return InSourceFormatName;
			}
			UE_LOGF(LogMetaSound, Error, "No source format set in MetaSound Environment for CAT Waveplayer node");
			// Fallthrough to default case.
		}
		default:
		{
			// No source format captured, so fallback to mono.
			return FName("Cat:Mono1Dot0");
		}
		}
	}

	// ---------------------------------------------------------------------
	// FCatWavePlayerOperator
	// ---------------------------------------------------------------------
	class FCatWavePlayerOperator final : public TExecutableOperator<FCatWavePlayerOperator>
	{
	public:
		struct FOperatorInputs
		{
			FOperatorInputs(
				FTriggerReadRef InPlay,
				FTriggerReadRef InStopAll,
				MetasoundCatExperimental::FSoundWaveContainerAssetReadRef InContainer,
				FFloatReadRef InPitchShift)
				: Play(MoveTemp(InPlay))
				, StopAll(MoveTemp(InStopAll))
				, Container(MoveTemp(InContainer))
				, PitchShift(MoveTemp(InPitchShift))
			{
			}

			FTriggerReadRef Play;
			FTriggerReadRef StopAll;
			MetasoundCatExperimental::FSoundWaveContainerAssetReadRef Container;
			FFloatReadRef PitchShift;

			// Index mode
			TOptional<FInt32ReadRef> Index;

			// Sequence mode
			TOptional<FInt32ReadRef> SequenceStartIndex;
			TOptional<FInt32ReadRef> NumSequenceLoops;
			TOptional<FTriggerReadRef> ResetSequence;

			// Random mode
			TOptional<FInt32ReadRef> Seed;
			TOptional<FInt32ReadRef> AvoidRepeatLast;
		};

		FCatWavePlayerOperator(
			const FBuildOperatorParams& InParams,
			FOperatorInputs&& InInputs,
			const FCatWavePlayerOperatorData& InConfig,
			FName InResolvedOutputFormat,
			FName InSourceFormatName)
			: Inputs(MoveTemp(InInputs))
			, Config(InConfig)
			, ResolvedOutputFormat(InResolvedOutputFormat)
			, SourceFormatName(InSourceFormatName)
			, Settings(InParams.OperatorSettings)
			, OutputAudio(FChannelAgnosticTypeWriteRef::CreateNew(InParams.OperatorSettings, InResolvedOutputFormat))
			, OnVoicesStarted          (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnVoicesFinished         (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnActiveVoiceCountChanged(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, ActiveVoiceCount         (FInt32WriteRef::CreateNew(0))
			, OnSequenceFirstSound     (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnSequenceLastSound      (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnSequenceLooped         (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnSequenceFinished       (FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, TranscodeScratch(Audio::GetChannelRegistry().FindConcreteChannelChecked(InResolvedOutputFormat), InParams.OperatorSettings.GetNumFramesPerBlock())
		{
			Voices.SetNum(FMath::Max(1, Config.MaxVoices));
			for (FInternalVoice& Voice : Voices)
			{
				Voice.Initialize(static_cast<float>(Settings.GetSampleRate()), Settings.GetNumFramesPerBlock());
			}

			// Seed transcoders from the initial container snapshot.
			RebuildTranscodersFromContainer();
		}

		virtual ~FCatWavePlayerOperator() override = default;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto Create = []() -> FNodeClassMetadata
			{
				FCatWavePlayerOperatorData DefaultConfig;
				FNodeClassMetadata Metadata
				{
					FNodeClassName{ TEXT("Experimental"), TEXT("Wave Player"), TEXT("Audio") },
					1, 0,
					METASOUND_LOCTEXT("CatWavePlayerDisplayName", "Wave Player (Audio)"),
					METASOUND_LOCTEXT("CatWavePlayerDescription", "Polyphonic multichannel-aware wave player sourced from a Sound Wave Container."),
					PluginAuthor,
					PluginNodeMissingPrompt,
					GetVertexInterface(DefaultConfig),
					{ NodeCategories::Generators },
					{
						METASOUND_LOCTEXT("Metasound_CatWavePlayerKeyword", "Wave Player"),
						METASOUND_LOCTEXT("Metasound_CatWavePlayerKeyword_Audio", "audio"),
						METASOUND_LOCTEXT("Metasound_CatWavePlayerKeyword_Multichannel", "multichannel"),
						METASOUND_LOCTEXT("Metasound_CatWavePlayerKeyword_Polyphonic", "Polyphonic"),
					},
					FNodeDisplayStyle{}
				};
				return Metadata;
			};
			static const FNodeClassMetadata Metadata = Create();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			// Pull config out of the node's operator data. Defaults apply when the
			// graph stored a bare node with no configuration override.
			FCatWavePlayerOperatorData Config;
			if (const FCatWavePlayerOperatorData* Found = CastOperatorData<const FCatWavePlayerOperatorData>(InParams.Node.GetOperatorData().Get()))
			{
				Config = *Found;
			}

			FOperatorInputs Inputs(
				InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputPlay),    InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputStopAll), InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<MetasoundCatExperimental::FSoundWaveContainerAsset>(METASOUND_GET_PARAM_NAME(InputSoundWaveContainer), InParams.OperatorSettings),
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPitchShift), InParams.OperatorSettings));

			if (Config.PlaybackMode == ECatWavePlayerPlaybackMode::Standard && Config.PlaybackType == ECatWavePlayerPlaybackType::Index)
			{
				Inputs.Index = InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputIndex), InParams.OperatorSettings);
			}

			if (Config.PlaybackType == ECatWavePlayerPlaybackType::Sequence)
			{
				Inputs.SequenceStartIndex = InParams.InputData.GetOrCreateDefaultDataReadReference<int32>    (METASOUND_GET_PARAM_NAME(InputSequenceStartIndex), InParams.OperatorSettings);
				Inputs.NumSequenceLoops   = InParams.InputData.GetOrCreateDefaultDataReadReference<int32>    (METASOUND_GET_PARAM_NAME(InputNumSequenceLoops),   InParams.OperatorSettings);
				Inputs.ResetSequence      = InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger> (METASOUND_GET_PARAM_NAME(InputResetSequence),      InParams.OperatorSettings);
			}

			if (Config.PlaybackMode == ECatWavePlayerPlaybackMode::Random)
			{
				Inputs.Seed            = InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputSeed),            InParams.OperatorSettings);
				Inputs.AvoidRepeatLast = InParams.InputData.GetOrCreateDefaultDataReadReference<int32>(METASOUND_GET_PARAM_NAME(InputAvoidRepeatLast), InParams.OperatorSettings);
			}

			// Capture the source-side format in case needed later
			using namespace Metasound::Frontend;
			const FName CapturedSourceFormatName =
				InParams.Environment.Contains<FName>(SourceInterface::Environment::SourceFormatName)
					? InParams.Environment.GetValue<FName>(SourceInterface::Environment::SourceFormatName)
					: NAME_None;

			// Resolve output format against current container + captured source format.
			TSharedPtr<const FCatSoundWaveContainerProxy> ContainerProxy = Inputs.Container->IsValid() ? Inputs.Container->GetLatest() : nullptr;
			const FName ResolvedFormat = ResolveOutputFormat(Config, ContainerProxy, CapturedSourceFormatName, &OutResults);

			return MakeUnique<FCatWavePlayerOperator>(InParams, MoveTemp(Inputs), Config, ResolvedFormat, CapturedSourceFormatName);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPlay),               Inputs.Play);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStopAll),            Inputs.StopAll);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSoundWaveContainer), Inputs.Container);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPitchShift),         Inputs.PitchShift);

			if (Inputs.Index.IsSet())
			{
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputIndex), *Inputs.Index);
			}

			if (Inputs.SequenceStartIndex.IsSet())
			{
				check(Inputs.NumSequenceLoops.IsSet());
				check(Inputs.ResetSequence.IsSet());
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSequenceStartIndex), *Inputs.SequenceStartIndex);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputNumSequenceLoops),   *Inputs.NumSequenceLoops);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetSequence),      *Inputs.ResetSequence);
			}

			if (Inputs.Seed.IsSet())
			{
				check(Inputs.AvoidRepeatLast.IsSet());
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSeed),            *Inputs.Seed);
				InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAvoidRepeatLast), *Inputs.AvoidRepeatLast);
			}

			// Re-resolve the output format now that inputs are bound. Matches the
			// Ladder Filter / Multiply pattern (UpdateCatFormat from BindInputs).
			UpdateCatFormat();
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputAudio),                     OutputAudio);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnVoicesStarted),           OnVoicesStarted);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnVoicesFinished),          OnVoicesFinished);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnActiveVoiceCountChanged), OnActiveVoiceCountChanged);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputActiveVoiceCount),          ActiveVoiceCount);

			if (Config.PlaybackType == ECatWavePlayerPlaybackType::Sequence)
			{
				InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnSequenceFirstSound), OnSequenceFirstSound);
				InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnSequenceLastSound),  OnSequenceLastSound);
				InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnSequenceLooped),     OnSequenceLooped);
				InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnSequenceFinished),   OnSequenceFinished);
			}
		}

		void Reset(const IOperator::FResetParams& InResetParams)
		{
			Settings = InResetParams.OperatorSettings;
			OutputAudio->Zero();
			OnVoicesStarted->Reset();
			OnVoicesFinished->Reset();
			OnActiveVoiceCountChanged->Reset();
			OnSequenceFirstSound->Reset();
			OnSequenceLastSound->Reset();
			OnSequenceLooped->Reset();
			OnSequenceFinished->Reset();
			*ActiveVoiceCount = 0;

			for (FInternalVoice& Voice : Voices)
			{
				Voice.StopPlaying();
			}
			PrevActiveVoiceCount = 0;
			NextActivationOrder = 0;
			SequencePosition = 0;
			SequenceLoopsRemaining = -1;
			LastHandledResetFrame = -1;
			bRandomSeeded = false;
			CachedContainerProxy.Reset();
			RecentRandomRingHead = 0;
			RecentRandomRingCount = 0;

			RebuildTranscodersFromContainer();
		}

		void Execute()
		{
			OnVoicesStarted->AdvanceBlock();
			OnVoicesFinished->AdvanceBlock();
			OnActiveVoiceCountChanged->AdvanceBlock();
			OnSequenceFirstSound->AdvanceBlock();
			OnSequenceLastSound->AdvanceBlock();
			OnSequenceLooped->AdvanceBlock();
			OnSequenceFinished->AdvanceBlock();
			LastHandledResetFrame = -1;

			const int32 NumFrames = Settings.GetNumFramesPerBlock();

			// Pick up any new container version 
			TSharedPtr<const FCatSoundWaveContainerProxy> NewProxy = Inputs.Container->IsValid() ? Inputs.Container->GetLatest() : nullptr;
			if (NewProxy != CachedContainerProxy)
			{
				CachedContainerProxy = NewProxy;
				// Sequence state resets on container swap 
				SequencePosition = 0;
				SequenceLoopsRemaining = -1;
				RebuildTranscodersFromContainer();
			}

			// Zero output; mix voices in.
			OutputAudio->Zero();

			// Handle Stop All trigger — the trigger fires at sub-block granularity, but
			// voice stopping happens at the next block's top; Something to investigate 
			// in the future. 
			if (Inputs.StopAll->IsTriggeredInBlock())
			{
				for (FInternalVoice& Voice : Voices)
				{
					Voice.StopPlaying();
				}
			}

			// Reset sequence (Sequence mode only).
			if (Inputs.ResetSequence.IsSet() && (*Inputs.ResetSequence)->IsTriggeredInBlock())
			{
				SequencePosition = 0;
				SequenceLoopsRemaining = -1;
				LastHandledResetFrame = 0;
			}

			// Handle Play triggers.
			const int32 NumPlayTriggers = Inputs.Play->NumTriggeredInBlock();
			const int32 PrevActiveBeforePlay = CountActiveVoices();
			for (int32 TriggerIdx = 0; TriggerIdx < NumPlayTriggers; ++TriggerIdx)
			{
				DispatchPlayTrigger();
			}

			if (NumPlayTriggers > 0 && PrevActiveBeforePlay == 0)
			{
				OnVoicesStarted->TriggerFrame(0);
			}

			// Render active voices.
			for (FInternalVoice& Voice : Voices)
			{
				if (Voice.bIsPlaying)
				{
					Voice.RenderAudio(NumFrames);
					Voice.MixIntoOutput(*OutputAudio, Transcoders, TranscodeScratch, NumFrames);
				}
			}

			// Flip finished voices inactive so subsequent blocks can reuse the slot.
			for (FInternalVoice& Voice : Voices)
			{
				if (Voice.bIsPlaying && Voice.bIsFinished)
				{
					Voice.bIsPlaying = false;
				}
			}

			const int32 NowActive = CountActiveVoices();
			if (NowActive != PrevActiveVoiceCount)
			{
				OnActiveVoiceCountChanged->TriggerFrame(0);
			}
			if (NowActive == 0 && PrevActiveVoiceCount > 0)
			{
				OnVoicesFinished->TriggerFrame(0);
			}
			PrevActiveVoiceCount = NowActive;
			*ActiveVoiceCount = NowActive;
		}

	private:
		int32 CountActiveVoices() const
		{
			int32 Count = 0;
			for (const FInternalVoice& Voice : Voices)
			{
				if (Voice.bIsPlaying)
				{
					++Count;
				}
			}
			return Count;
		}

		FInternalVoice* FindInactiveVoiceOrStealOldest()
		{
			FInternalVoice* Oldest = nullptr;
			for (FInternalVoice& Voice : Voices)
			{
				if (!Voice.bIsPlaying)
				{
					return &Voice;
				}
				if (!Oldest || Voice.ActivationOrder < Oldest->ActivationOrder)
				{
					Oldest = &Voice;
				}
			}
			return Oldest;
		}

		void DispatchPlayTrigger()
		{
			if (!CachedContainerProxy.IsValid())
			{
				return;
			}
			const FCatSoundWaveContainerData& Data = CachedContainerProxy->GetData();
			if (Data.Entries.IsEmpty())
			{
				return;
			}

			int32 ChosenIndex = INDEX_NONE;
			const bool bSequenceMode = Config.PlaybackType == ECatWavePlayerPlaybackType::Sequence;
			const bool bRandomMode   = Config.PlaybackMode == ECatWavePlayerPlaybackMode::Random;

			if (bSequenceMode)
			{
				const int32 NumEntries = Data.Entries.Num();
				const int32 StartOffset = Inputs.SequenceStartIndex.IsSet() ? **Inputs.SequenceStartIndex : 0;
				const int32 NumLoops    = Inputs.NumSequenceLoops.IsSet()   ? **Inputs.NumSequenceLoops   : -1;
				// NumLoops == 0: play nothing (explicit opt-out).
				// NumLoops == 1: play the sequence once.
				// NumLoops  < 0: loop forever.
				if (NumLoops == 0)
				{
					return;
				}
				if (SequenceLoopsRemaining < 0 && NumLoops >= 0)
				{
					SequenceLoopsRemaining = NumLoops;
				}

				if (SequencePosition == 0)
				{
					OnSequenceFirstSound->TriggerFrame(0);
				}

				int32 WrappedIdx = (StartOffset + SequencePosition) % NumEntries;
				if (WrappedIdx < 0) WrappedIdx += NumEntries;
				ChosenIndex = WrappedIdx;

				++SequencePosition;
				if (SequencePosition >= NumEntries)
				{
					SequencePosition = 0;
					if (NumLoops > 0)
					{
						--SequenceLoopsRemaining;
						if (SequenceLoopsRemaining <= 0)
						{
							OnSequenceLastSound->TriggerFrame(0);
							OnSequenceFinished->TriggerFrame(0);
						}
						else
						{
							OnSequenceLooped->TriggerFrame(0);
						}
					}
					else if (NumLoops < 0)
					{
						OnSequenceLooped->TriggerFrame(0);
					}
				}
			}
			else if (bRandomMode)
			{
				ChosenIndex = PickRandomIndex(Data);
			}
			else
			{
				const int32 IndexRequested = Inputs.Index.IsSet() ? **Inputs.Index : 0;
				const int32 Clamped = FMath::Clamp(IndexRequested, 0, Data.Entries.Num() - 1);
				ChosenIndex = Clamped;
			}

			if (ChosenIndex == INDEX_NONE || !Data.Entries.IsValidIndex(ChosenIndex))
			{
				return;
			}
			const FCatSoundWaveContainerData::FEntry& Entry = Data.Entries[ChosenIndex];
			if (!Entry.SoundWave.IsValid() || Entry.Weight <= 0.0f)
			{
				return;
			}

			// sanitize before passing so a NaN on the input doesn't poison the
			// resampler. 
			const float NodePitchRaw = *Inputs.PitchShift;
			const float TotalPitch   = FMath::IsFinite(NodePitchRaw) ? NodePitchRaw : 0.0f;

			FInternalVoice* Voice = FindInactiveVoiceOrStealOldest();
			if (!Voice)
			{
				return;
			}
			Voice->StartPlaying(
				Entry.SoundWave,
				TotalPitch,
				static_cast<float>(Settings.GetSampleRate()),
				Settings.GetNumFramesPerBlock(),
				++NextActivationOrder);
		}

		int32 PickRandomIndex(const FCatSoundWaveContainerData& Data)
		{
			if (!bRandomSeeded)
			{
				const int32 Seed = Inputs.Seed.IsSet() ? **Inputs.Seed : -1;
				RandomStream.Initialize(Seed < 0 ? FMath::Rand() : Seed);
				bRandomSeeded = true;
			}

			const int32 AvoidCount = Inputs.AvoidRepeatLast.IsSet() ? **Inputs.AvoidRepeatLast : 0;
			const int32 NumEntries = Data.Entries.Num();

			ScratchCandidates.Reset();
			ScratchWeights.Reset();

			// Resize scratch if the container grew beyond our prior capacity (initial
			// sizing happened in RebuildTranscodersFromContainer()). Reserve only grows
			// the underlying allocation; no-op when already sized.
			ScratchCandidates.Reserve(NumEntries);
			ScratchWeights.Reserve(NumEntries);

			float TotalWeight = 0.0f;
			for (int32 i = 0; i < NumEntries; ++i)
			{
				if (Data.Entries[i].Weight <= 0.0f || !Data.Entries[i].SoundWave.IsValid())
				{
					continue;
				}
				if (IsInRecentWindow(i))
				{
					continue;
				}
				ScratchCandidates.Add(i);
				ScratchWeights.Add(Data.Entries[i].Weight);
				TotalWeight += Data.Entries[i].Weight;
			}
			if (ScratchCandidates.IsEmpty())
			{
				return INDEX_NONE;
			}

			const float Draw = RandomStream.FRandRange(0.0f, TotalWeight);
			float Accum = 0.0f;
			int32 PickedCandidate = ScratchCandidates.Num() - 1;
			for (int32 i = 0; i < ScratchCandidates.Num(); ++i)
			{
				Accum += ScratchWeights[i];
				if (Draw <= Accum)
				{
					PickedCandidate = i;
					break;
				}
			}
			const int32 PickedIndex = ScratchCandidates[PickedCandidate];

			// ring-buffer recency window. No element shifting. Capacity is fixed to
			// the entry count so MaxWindow is always bounded.
			const int32 MaxWindow = FMath::Clamp(AvoidCount, 0, FMath::Max(0, NumEntries - 1));
			PushRecentIndex(PickedIndex, MaxWindow);

			return PickedIndex;
		}

		// Recency ring: RecentRandomIndices is used as a fixed-capacity circular
		// buffer. RecentRandomRingHead is the next write slot; RecentRandomRingCount
		// is the current fill level (<= RecentRandomIndices.Num() which is the capacity).
		bool IsInRecentWindow(int32 Index) const
		{
			for (int32 i = 0; i < RecentRandomRingCount; ++i)
			{
				const int32 Capacity = RecentRandomIndices.Num();
				if (Capacity <= 0)
				{
					break;
				}
				const int32 Slot = ((RecentRandomRingHead - 1 - i) % Capacity + Capacity) % Capacity;
				if (RecentRandomIndices[Slot] == Index)
				{
					return true;
				}
			}
			return false;
		}

		void PushRecentIndex(int32 Index, int32 MaxWindow)
		{
			if (MaxWindow <= 0)
			{
				RecentRandomRingCount = 0;
				RecentRandomRingHead = 0;
				return;
			}
			const int32 Capacity = RecentRandomIndices.Num();
			if (Capacity < MaxWindow)
			{
				// Shouldn't happen after RebuildTranscodersFromContainer sizes the ring,
				// but grow defensively (one-time alloc if the container changed size).
				RecentRandomIndices.SetNum(MaxWindow);
				RecentRandomRingHead = FMath::Min(RecentRandomRingHead, MaxWindow);
				RecentRandomRingCount = FMath::Min(RecentRandomRingCount, MaxWindow);
			}
			else if (Capacity > MaxWindow)
			{
				// Window shrank (AvoidCount lowered). Drop oldest entries by capping the
				// live count; the ring capacity stays at its high-water mark.
				RecentRandomRingCount = FMath::Min(RecentRandomRingCount, MaxWindow);
			}
			const int32 UsableCapacity = FMath::Max(1, MaxWindow);
			RecentRandomIndices[RecentRandomRingHead % UsableCapacity] = Index;
			RecentRandomRingHead = (RecentRandomRingHead + 1) % UsableCapacity;
			RecentRandomRingCount = FMath::Min(RecentRandomRingCount + 1, UsableCapacity);
		}

		// Re-resolve the output format against the currently-bound container and the
		// source-format hint captured at CreateOperator time. 
		void UpdateCatFormat()
		{
			const TSharedPtr<const FCatSoundWaveContainerProxy> ContainerProxy =
				Inputs.Container->IsValid() ? Inputs.Container->GetLatest() : nullptr;
			const FName NewFormat = ResolveOutputFormat(Config, ContainerProxy, SourceFormatName, /*OutResults*/ nullptr);
			if (NewFormat != ResolvedOutputFormat)
			{
				ResolvedOutputFormat = NewFormat;
				OutputAudio = FChannelAgnosticTypeWriteRef::CreateNew(Settings, ResolvedOutputFormat);
				TranscodeScratch = Audio::FChannelAgnosticType(Audio::GetChannelRegistry().FindConcreteChannelChecked(ResolvedOutputFormat), Settings.GetNumFramesPerBlock());
				RebuildTranscodersFromContainer();
			}
		}

		void RebuildTranscodersFromContainer()
		{
			Transcoders.Reset();
			if (!CachedContainerProxy.IsValid())
			{
				return;
			}
			const FCatSoundWaveContainerData& Data = CachedContainerProxy->GetData();

			const int32 NumEntries = Data.Entries.Num();
			const int32 RingCap = FMath::Max(0, NumEntries - 1);
			if (RecentRandomIndices.Num() < RingCap)
			{
				RecentRandomIndices.SetNumUninitialized(RingCap);
			}
			RecentRandomRingHead = FMath::Min(RecentRandomRingHead, FMath::Max(1, RingCap));
			RecentRandomRingCount = FMath::Min(RecentRandomRingCount, RingCap);
			if (ScratchCandidates.Max() < NumEntries)
			{
				ScratchCandidates.Reserve(NumEntries);
			}
			if (ScratchWeights.Max() < NumEntries)
			{
				ScratchWeights.Reserve(NumEntries);
			}

			for (const FCatSoundWaveContainerData::FEntry& Entry : Data.Entries)
			{
				if (!Entry.SoundWave.IsValid())
				{
					continue;
				}
				const int32 NumSrcChannels = FMath::Max<int32>(1, Entry.SoundWave->GetSoundWaveDataRef()->GetNumChannels());
				if (Transcoders.Contains(NumSrcChannels))
				{
					continue;
				}

				// Build a probe CAT at the source channel count so the transcoder
				// resolver has both endpoints. The probe is immediately discarded;
				// only the resolved FTranscoder callable is kept.
				const FName SrcFormat = NumChannelsToCatFormat(NumSrcChannels);
				Metasound::FChannelAgnosticType ProbeSrc(Settings, SrcFormat);
				Audio::ChannelAgnosticTranscoder::FTranscoder Transcoder =
					Audio::FTranscoderResolver::Resolve(
						ProbeSrc.GetType(),
						{
							.ToType = OutputAudio->GetType(),
							.TranscodeMethod = Audio::EChannelTranscodeMethod::MixUpOrDown,
							.MixMethod = Audio::EChannelMapMonoUpmixMethod::FullVolume
						});
				Transcoders.Emplace(NumSrcChannels, MoveTemp(Transcoder));
			}
		}

		FOperatorInputs Inputs;
		FCatWavePlayerOperatorData Config;
		FName ResolvedOutputFormat;
		FName SourceFormatName;
		FOperatorSettings Settings;

		FChannelAgnosticTypeWriteRef OutputAudio;
		FTriggerWriteRef OnVoicesStarted;
		FTriggerWriteRef OnVoicesFinished;
		FTriggerWriteRef OnActiveVoiceCountChanged;
		FInt32WriteRef   ActiveVoiceCount;
		FTriggerWriteRef OnSequenceFirstSound;
		FTriggerWriteRef OnSequenceLastSound;
		FTriggerWriteRef OnSequenceLooped;
		FTriggerWriteRef OnSequenceFinished;

		TArray<FInternalVoice> Voices;
		int32 PrevActiveVoiceCount = 0;
		uint32 NextActivationOrder = 0;

		/** Latest container proxy seen on the audio thread; refreshed via GetLatest() each block. */
		TSharedPtr<const FCatSoundWaveContainerProxy> CachedContainerProxy;

		/** Transcoder keyed by source channel count. Rebuilt on container swap */
		TMap<int32, Audio::ChannelAgnosticTranscoder::FTranscoder> Transcoders;
		Audio::FChannelAgnosticType TranscodeScratch;

		// Sequence state.
		int32 SequencePosition = 0;
		int32 SequenceLoopsRemaining = -1;
		int32 LastHandledResetFrame = -1;

		// Random state.
		FRandomStream RandomStream;
		bool bRandomSeeded = false;

		// recency ring + candidate/weight scratch. Sized to the container's
		// entry count in RebuildTranscodersFromContainer(); 
		TArray<int32> RecentRandomIndices;
		int32 RecentRandomRingHead = 0;
		int32 RecentRandomRingCount = 0;
		TArray<int32> ScratchCandidates;
		TArray<float> ScratchWeights;
	};

} // namespace Metasound::CatWavePlayer

namespace Metasound
{
	using FCatWavePlayerNode = TNodeFacade<Metasound::CatWavePlayer::FCatWavePlayerOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatWavePlayerNode, FMetasoundCatWavePlayerNodeConfiguration);
}

FMetasoundCatWavePlayerNodeConfiguration::FMetasoundCatWavePlayerNodeConfiguration() = default;

TInstancedStruct<FMetasoundFrontendClassInterface> FMetasoundCatWavePlayerNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound::CatWavePlayer;
	FCatWavePlayerOperatorData Data;
	Data.MaxVoices    = FMath::Max(1, MaxVoices);
	Data.Format       = Format;
	Data.CustomFormat = CustomFormat;
	Data.PlaybackType = PlaybackType;
	Data.PlaybackMode = PlaybackMode;

	// The Audio Out vertex is declared with the polymorphic base CAT type by GetVertexInterface.
	// Concrete format resolution happens at CreateOperator time against the actual bound
	// container + environment; any resulting concrete format (Cat:Mono1Dot0, Cat:Stereo2Dot0,
	// Cat:Surround5Dot1, ...) is castable to the base type via IsCastable, so bind succeeds
	// regardless of the resolution outcome.
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(
		FMetasoundFrontendClassInterface::GenerateClassInterface(
			Metasound::CatWavePlayer::GetVertexInterface(Data)));
}

TSharedPtr<const Metasound::IOperatorData> FMetasoundCatWavePlayerNodeConfiguration::GetOperatorData() const
{
	using namespace Metasound::CatWavePlayer;
	TSharedRef<FCatWavePlayerOperatorData> Data = MakeShared<FCatWavePlayerOperatorData>();
	Data->MaxVoices    = FMath::Max(1, MaxVoices);
	Data->Format       = Format;
	Data->CustomFormat = CustomFormat;
	Data->PlaybackType = PlaybackType;
	Data->PlaybackMode = PlaybackMode;
	return Data;
}

#undef LOCTEXT_NAMESPACE
