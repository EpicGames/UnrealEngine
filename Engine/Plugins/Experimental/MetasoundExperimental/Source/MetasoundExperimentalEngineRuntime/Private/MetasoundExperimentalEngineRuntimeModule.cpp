// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExperimentalEngineRuntimeModule.h"

#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "Modules/ModuleManager.h"

void FMetasoundExperimentalEngineRuntimeModule::StartupModule()
{
	FModuleManager::Get().LoadModule("AudioExperimentalRuntime");
	METASOUND_REGISTER_ITEMS_IN_MODULE
}

void FMetasoundExperimentalEngineRuntimeModule::ShutdownModule()
{
	METASOUND_UNREGISTER_ITEMS_IN_MODULE
}

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(FMetasoundExperimentalEngineRuntimeModule, MetasoundExperimentalEngineRuntime);

