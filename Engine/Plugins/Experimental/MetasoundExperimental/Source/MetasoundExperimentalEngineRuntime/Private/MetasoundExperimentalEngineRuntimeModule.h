// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


class FMetasoundExperimentalEngineRuntimeModule : public IModuleInterface
{ 
public:
	//~ Begin IModuleInterface API
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface API

private:
};
