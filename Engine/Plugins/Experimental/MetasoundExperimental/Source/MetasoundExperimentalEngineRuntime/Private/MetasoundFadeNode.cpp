// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundFadeNode.h"

#include "DSP/Dsp.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"

#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundDataFactory.h"
#include "MetasoundEnumRegistrationMacro.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "MetasoundDataReference.h"
#include "MetasoundTrigger.h"
#include "MetasoundTime.h"
#include "MetasoundPrimitives.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundVertex.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundSampleCounter.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalNodes_Fade"

namespace Metasound
{
	namespace FadeNodePrivate
	{
		// List of Curve Functions
		enum class ECurveFunction
		{
			Linear,
			EqualPower,
			EqualPowerInverted,
			SCurve,
			SCurveInverted,
			Logarithmic,
			LogarithmicInverted,
			Exponential,
			ExponentialInverted
		};

		// Functions for manipulating curves
		struct FCurveUtilities
		{
			// Function for generating a curve segment given the percentage range (0.f - 1.f), output clamped between 0.f and 1.f. InExponent only used if InCurveFunction is of an Exponential type.
			static void GetValue(float* OutBuffer, const int32 InNumSamples, const float InStartPercent, const float InEndPercent, const ECurveFunction InCurveFunction, const float InExponent = 1.f)
			{
				TArrayView<float> OutSamples(OutBuffer, InNumSamples);

				if (InNumSamples == 0)
				{
					return;
				}

				const float PerSamplePercentFraction = (InEndPercent - InStartPercent) / static_cast<float>(InNumSamples);

				float CurrentPercentage = InStartPercent;

				switch (InCurveFunction)
				{

				case ECurveFunction::Linear:
				{
					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = CurrentPercentage;
					}

					break;
				}

				case ECurveFunction::EqualPower:
				{
					const float HalfPi = 0.5 * PI;

					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = FMath::Cos(CurrentPercentage * HalfPi - HalfPi);
					}

					break;
				}

				case ECurveFunction::EqualPowerInverted:
				{
					const float HalfPi = 0.5 * PI;

					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = FMath::Cos(CurrentPercentage * HalfPi + PI) + 1.f;
					}

					break;
				}

				case ECurveFunction::SCurve:
				{
					float X = 0.f;
					float XPow = 0.f;

					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						X = (6.f * CurrentPercentage) - 3.f;

						XPow = FMath::Pow(3.f, 2.f * X);

						OutSamples[i] = 1.f - (1.f / (1.f + XPow));
					}

					break;
				}

				case ECurveFunction::SCurveInverted:
				{
					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = 4.f * FMath::Pow((CurrentPercentage - 0.5f), 3.f) + 0.5f;
					}

					break;
				}

				case ECurveFunction::Logarithmic:
				{
					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = FMath::LogX(10.f, (0.9f * CurrentPercentage + 0.1f)) + 1.f;
					}

					break;
				}

				case ECurveFunction::LogarithmicInverted:
				{
					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = FMath::LogX(0.1f, (-0.9f * CurrentPercentage + 1.f));
					}

					break;
				}

				case ECurveFunction::Exponential:
				{
					const float Exponent = FMath::Max(SMALL_NUMBER, InExponent);

					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						OutSamples[i] = FMath::Pow(CurrentPercentage, Exponent);
					}

					break;
				}

				case ECurveFunction::ExponentialInverted:
				{
					const float Exponent = FMath::Max(SMALL_NUMBER, InExponent);

					float Base = 1.f;

					for (int32 i = 0; i < InNumSamples; ++i, CurrentPercentage += PerSamplePercentFraction)
					{
						Base = 1 - CurrentPercentage;

						OutSamples[i] = -FMath::Pow(Base, Exponent) + 1.f;
					}

					break;
				}
				}

				// Clamp output values
				Audio::ArrayClampInPlace(OutSamples, 0.f, 1.f);
			}
		};

		// Fade data needed to define the parameters of the fade
		struct FFadeData
		{
			// Total number of samples in the fade
			int32 FadeSamples = 1;

			// Output fade value on start
			float StartValue = 0.f;

			// Output fade value by end
			float EndValue = 1.f;

			// Exponent when Curve Function is set to Exponential
			float Exponent = 1.f;

			// Curve Function
			ECurveFunction CurveFunction = ECurveFunction::EqualPower;

			// Whether or not to reset the fade output value once fade has ended
			bool bResetValue = false;
		};

		struct FFade
		{
			// Initialize Fade
			void Init(const FFadeData& InData)
			{
				Data = InData;

				StartIndex = INDEX_NONE;
				CurrentIndex = INDEX_NONE;

				bActive = false;
				bPlayed = false;
			}

			// Start fade for generation
			void Start(const int32 InStartIndex = 0)
			{
				bActive = true;
				bPlayed = true;
				StartIndex = InStartIndex;
				CurrentIndex = InStartIndex;
			}

			// Update fade data, make sure Current Index is clamped within the fade duration
			void SetFadeData(const FFadeData& InData)
			{
				if (bActive)
				{
					CurrentIndex = FMath::Clamp(CurrentIndex, 0, InData.FadeSamples);
				}

				Data = InData;
			}

			/* Generate fade segment
			* @param OutFloat pointer to generated segment
			* @param InNumSamplesToGenerate number of samples to generate
			* @param bIncrementIndex updates CurrentIndex by the number of generated samples if true
			*/ 
			void GenerateSegment(float* OutFloat, const int32 InNumSamplesToGenerate, const bool bIncrementIndex = true)
			{
				// Determine actual number of samples to generate
				const int32 ActualSamplesToGenerate = Data.FadeSamples > (CurrentIndex + InNumSamplesToGenerate) ? InNumSamplesToGenerate : Data.FadeSamples - CurrentIndex;

				// Determine if there are any samples within the given range that are not part of the function generation (left over after finishing the fade)
				const int32 SamplesRemaining = InNumSamplesToGenerate - ActualSamplesToGenerate;

				// Safety check that sample count adds up
				check(SamplesRemaining >= 0 && ActualSamplesToGenerate + SamplesRemaining == InNumSamplesToGenerate);

				// If ActualSamplesToGenerate is less than or equal to 0, then the fade is complete and Active is set to false
				if (ActualSamplesToGenerate <= 0)
				{
					bActive = false;
				}

				// If the Fade is not active by this point, all generated samples are either the start or end value
				if (!bActive)
				{
					// Depending on the ResetValue bool and whether the Fade has already played, generate a buffer of the correct input value
					const float InactiveValue = Data.bResetValue || !bPlayed ? Data.StartValue : Data.EndValue;

					// Create TArrayView of float pointer
					TArrayView<float> Value(OutFloat, InNumSamplesToGenerate);

					// Set Array value
					Audio::ArraySetToConstantInplace(Value, InactiveValue);

					// Early out
					return;
				}

				// Setup temporary buffer
				TArray<float> Value;
				Value.SetNum(ActualSamplesToGenerate + SamplesRemaining);

				// Get View of total scratch buffer
				TArrayView<float> ValueView(Value);

				// Create slice of scratch buffer view that is part of the fade segment
				TArrayView<float> FadeValueView = ValueView.Slice(0, ActualSamplesToGenerate);

				// Create slice of scratch buffer view that remains after finishing the fade segment
				TArrayView<float> RemainingValueView = ValueView.Slice(ActualSamplesToGenerate - 1, SamplesRemaining);

				// Calculate start and end percentage for normalized curve function
				const float StartPercentage = static_cast<float>(CurrentIndex) / static_cast<float>(Data.FadeSamples);
				const float EndPercentage = static_cast<float>(CurrentIndex + ActualSamplesToGenerate) / static_cast<float>(Data.FadeSamples);

				// Generate normalized fade
				FCurveUtilities::GetValue(FadeValueView.GetData(), FadeValueView.Num(), StartPercentage, EndPercentage, Data.CurveFunction, Data.Exponent);

				// Transform fade to appropriate values
				const float ValueOffset = Data.StartValue;
				const float ValueScale = Data.EndValue - Data.StartValue;
				Audio::ArrayMultiplyByConstantInPlace(FadeValueView, ValueScale);
				Audio::ArrayAddConstantInplace(FadeValueView, ValueOffset);

				// If remaining samples, generate const
				if (RemainingValueView.Num() > 0)
				{
					const float InactiveValue = Data.bResetValue ? Data.StartValue : Data.EndValue;

					Audio::ArraySetToConstantInplace(RemainingValueView, InactiveValue);
				}

				// Copy samples out
				memcpy(OutFloat, Value.GetData(), Value.GetAllocatedSize());

				// Update index
				CurrentIndex = bIncrementIndex ? ActualSamplesToGenerate + CurrentIndex : CurrentIndex;
			}

			// Returns if fade is currently Active
			bool IsActive() const
			{
				return bActive;
			}

			// Look ahead function determining if the fade render will complete within the given range based on its current index.
			// Returns positive frame value if within range, otherwise returns -1 if fade doesn't complete within range.
			int32 DoneWithinInFrameRange(const int32 InFrameRange)
			{
				if (Data.FadeSamples < InFrameRange + CurrentIndex)
				{
					return Data.FadeSamples - CurrentIndex;
				}

				return INDEX_NONE;
			}

		private:
			// Fade data defining the fade parameters
			FFadeData Data;

			// Playback state tracking
			int32 StartIndex = INDEX_NONE;
			int32 CurrentIndex = INDEX_NONE;

			// Active is true if fade is currently within its curve function playback
			bool bActive = false;

			// Played is true after the first play, determines whether output adheres to Start or End Values based on the reset value
			bool bPlayed = false;
		};
	

		namespace FadeVertexNames
		{
			METASOUND_PARAM(InputTrigger, "Trigger", "Trigger to start the fade.");
			METASOUND_PARAM(InputReset, "Reset", "Resets fade state to pre-played state.");
			METASOUND_PARAM(InputDuration, "Duration", "Duration time of the fade in seconds.");
			METASOUND_PARAM(InputStartValue, "Start Value", "Starting fade value.");
			METASOUND_PARAM(InputEndValue, "End Value", "Ending fade value.");
			METASOUND_PARAM(InputCurveFunction, "Curve Function", "Determines the shape of the curve.")
			METASOUND_PARAM(InputExponent, "Curve Exponent", "Exponent used when the Curve Function is an Exponential type.");
			METASOUND_PARAM(InputStartTime, "Start Time", "Number of seconds into the fade to start playback.");
			METASOUND_PARAM(InputResetValue, "Reset Value On Done", "When the fade is done, resets output value to start; outputs end value when false.");

			METASOUND_PARAM(OutputOnTrigger, "On Trigger", "Triggers when the fade is triggered.");
			METASOUND_PARAM(OutputOnDone, "On Done", "Triggers when the fade is finished.");
			METASOUND_PARAM(OutputOnNearlyDone, "On Nearly Done", "Triggers when the fade is nearly finished.");

			FLazyName OutputBaseName{ "Value" };
#if WITH_EDITOR
			const FText OutputTooltip = LOCTEXT("Value_ToolTip", "The output value of the fade.");
#else
			const FText OutputTooltip = FText::GetEmpty();
#endif

			FName MakeOutputVertexName(const EMetaSoundFadeOutputType InOutputType)
			{
				FString NameAddOn;

				if (InOutputType == EMetaSoundFadeOutputType::FloatType)
				{
					NameAddOn = "Float";
				}
				else if (InOutputType == EMetaSoundFadeOutputType::AudioBufferType)
				{
					NameAddOn = "Audio";
				}

				FString Name = NameAddOn + " " + OutputBaseName.ToString();
				return FName(Name);
			}

			FOutputDataVertex MakeOutputDataVertex(const EMetaSoundFadeOutputType InOutputType)
			{
				FName OutputName = MakeOutputVertexName(InOutputType);
#if WITH_EDITOR
				const FText OutputDisplayName = FText::Format(LOCTEXT("FadeValueOut_DisplayName", "{0}"), FText::FromName(OutputName));
#else
				const FText OutputDisplayName = FText::GetEmpty();
#endif
				FOutputDataVertex VertexData;

				if (InOutputType == EMetaSoundFadeOutputType::AudioBufferType)
				{
					VertexData = TOutputDataVertex<FAudioBuffer>{ OutputName, FDataVertexMetadata{ OutputTooltip, OutputDisplayName } };
				}
				else
				{
					VertexData = TOutputDataVertex<float>{ OutputName, FDataVertexMetadata{ OutputTooltip, OutputDisplayName } };
				}

				return VertexData;
			}
		}
	}

	DECLARE_METASOUND_ENUM(FadeNodePrivate::ECurveFunction, FadeNodePrivate::ECurveFunction::EqualPower, 
		METASOUNDEXPERIMENTALENGINERUNTIME_API, FEnumCurveFunction, FEnumCurveFunctionTypeInfo, FEnumCurveFunctionReadRef, FEnumCurveFunctionWriteRef);

	DEFINE_METASOUND_ENUM_BEGIN(FadeNodePrivate::ECurveFunction, FEnumCurveFunction, "CurveFunction")

		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::Linear, "LinearDescription", "Linear", "LinearDescriptionTT", "A line"),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::EqualPower, "EqualPowerDescription", "Equal Power", "EqualPowerDescriptionTT", "A curve function that retains relative power, good for crossfading."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::EqualPowerInverted, "EqualPowerInvertedDescription", "Equal Power (Inverted)", "EqualPowerInvertedDescriptionTT", "Similar function to Equal Power function but with the axes swapped."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::SCurve, "SCurveDescription", "S-Curve", "SCurveDescriptionTT", "An S-Curve function approximating tanh"),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::SCurveInverted, "SCurveInverted", "S-Curve (Inverted)", "SCurveInvertedDescriptionTT", "Similar function to S-Curve but with the axes swapped."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::Logarithmic, "LogarithmicDescription", "Logarithmic", "LogarithmicDescriptionTT", "a logx function providing a logarithmic curve."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::LogarithmicInverted, "LogarithmicInvertedDescription", "Logarithmic (Inverted)", "LogarithmicInvertedDescriptionTT", "Similar function to the Logarithmic function, but with the axes swapped."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::Exponential, "ExponentialDescription", "Exponential", "ExponentialDescriptionTT", "Exponential curve function. With this function you can use the Exponential float value to adjust the curve sharpness."),
		DEFINE_METASOUND_ENUM_ENTRY(FadeNodePrivate::ECurveFunction::ExponentialInverted, "ExponentialInvertedDescription", "Exponential (Inverted)", "ExponentialInvertedDescriptionTT", "Same as the Exponential function, but with the axes swapped. With this function you can use the Exponential float value to adjust the curve sharpness."),
		DEFINE_METASOUND_ENUM_END()

	namespace FadeNodePrivate
	{
		FVertexInterface GetVertexInterface(const EMetaSoundFadeOutputType InOutputType)
		{
			using namespace FadeNodePrivate::FadeVertexNames;

			FInputVertexInterface InputInterface{
				FInputVertexInterface(
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger)),
					TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputReset)),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDuration), 1.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStartValue), 0.0f),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputEndValue), 1.0f),
					TInputDataVertex<FEnumCurveFunction>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputCurveFunction), (int32)FadeNodePrivate::ECurveFunction::EqualPower),
					TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputExponent), 2.0f),
					TInputDataVertex<FTime>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputStartTime), 0.0f),
					TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(InputResetValue), false)
			) };

			FOutputVertexInterface OutputInterface{
				FOutputVertexInterface(
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnTrigger)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputOnDone)),
					TOutputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA_ADVANCED(OutputOnNearlyDone))
			) };
			OutputInterface.Add(FadeNodePrivate::FadeVertexNames::MakeOutputDataVertex(InOutputType));

			return FVertexInterface
			{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}

	}

	/** To send data from a FMetaSoundFrontendNodeConfiguration to an IOperator, it should
	* be encapsulated in the form of a IOperatorData.
	*
	* The use of the TOperatorData provides some safety mechanisms for downcasting node configurations.
	*/
	class FFadeNodeOperatorData : public TOperatorData<FFadeNodeOperatorData>
	{
	public:
		// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
		// that the downcast is valid.
		static const FLazyName OperatorDataTypeName;

		FFadeNodeOperatorData(const EMetaSoundFadeOutputType InOutputType)
			: OutputType(InOutputType)
		{
		}

		const EMetaSoundFadeOutputType& GetOutputType() const
		{
			return OutputType;
		}

	private:

		EMetaSoundFadeOutputType OutputType; 
	};

	const FLazyName FFadeNodeOperatorData::OperatorDataTypeName = "FadeNodeOperatorData";


	/** TFadeNodeOperator
	*
	*  Defines a Simple Fade node operator.
	*/
	class TFadeNodeOperator : public TExecutableOperator<TFadeNodeOperator>
	{
	public:

		static const FNodeClassMetadata& GetNodeInfo()
		{
			auto CreateNodeClassMetadata = []() -> FNodeClassMetadata
				{
					const FName OperatorName = "Fade";
					const FText NodeDisplayName = METASOUND_LOCTEXT("FadeDisplayNamePattern", "Fade");
					const FText NodeDescription = METASOUND_LOCTEXT("FadeDesc", "Generates a fade envelope.");
					const FVertexInterface NodeInterface = FadeNodePrivate::GetVertexInterface(EMetaSoundFadeOutputType::FloatType);

					const FNodeClassMetadata Metadata
					{
						FNodeClassName { StandardNodes::Namespace, OperatorName, ""},
						1, // Major Version
						0, // Minor Version
						NodeDisplayName,
						NodeDescription,
						PluginAuthor,
						PluginNodeMissingPrompt,
						NodeInterface,
						{ NodeCategories::Functions },
						{
							METASOUND_LOCTEXT("Metasound_Fade", "Fade"),
							METASOUND_LOCTEXT("Metasound_FadeIn", "Fade In"),
							METASOUND_LOCTEXT("Metasound_FadeOut", "Fade Out"),
							METASOUND_LOCTEXT("Metasound_EnvelopeSegment", "Envelope Segment"),
							METASOUND_LOCTEXT("Metasound_MSEG", "MSEG"),
							METASOUND_LOCTEXT("Metasound_Ramp", "Ramp")
						},
						FNodeDisplayStyle()
					};

					return Metadata;
				};

			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata();
			return Metadata;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace FadeNodePrivate::FadeVertexNames;

			// Collect configuration data
			EMetaSoundFadeOutputType OutputType = EMetaSoundFadeOutputType::FloatType;
			if (const FFadeNodeOperatorData* FadeNodeConfig = CastOperatorData<const FFadeNodeOperatorData>(InParams.Node.GetOperatorData().Get()))
			{
				// Get Config Operator Data
				OutputType = FadeNodeConfig->GetOutputType();
			}

			const FInputVertexInterfaceData& InputData = InParams.InputData;

			FTriggerReadRef	TriggerIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
			FTriggerReadRef	ResetIn = InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputReset), InParams.OperatorSettings);
			FTimeReadRef	DurationIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputDuration), InParams.OperatorSettings);
			FFloatReadRef	StartValueIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputStartValue), InParams.OperatorSettings);
			FFloatReadRef	EndValueIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputEndValue), InParams.OperatorSettings);
			FEnumCurveFunctionReadRef	CurveFunctionIn = InputData.GetOrCreateDefaultDataReadReference<FEnumCurveFunction>(METASOUND_GET_PARAM_NAME(InputCurveFunction), InParams.OperatorSettings);
			FFloatReadRef	ExponentIn = InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputExponent), InParams.OperatorSettings);
			FTimeReadRef	StartTimeIn = InputData.GetOrCreateDefaultDataReadReference<FTime>(METASOUND_GET_PARAM_NAME(InputStartTime), InParams.OperatorSettings);
			FBoolReadRef	ResetValueIn = InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InputResetValue), InParams.OperatorSettings);

			return MakeUnique<TFadeNodeOperator>(InParams, 
				TriggerIn, 
				ResetIn,
				DurationIn, 
				StartValueIn, 
				EndValueIn, 
				StartTimeIn, 
				ResetValueIn,
				CurveFunctionIn,
				ExponentIn,
				OutputType
			);
		}

		TFadeNodeOperator(const FBuildOperatorParams& InParams,
			const FTriggerReadRef& InTriggerIn,
			const FTriggerReadRef& InResetIn,
			const FTimeReadRef& InDurationIn,
			const FFloatReadRef& InStartValueIn,
			const FFloatReadRef& InEndValueIn,
			const FTimeReadRef& InStartTimeIn,
			const FBoolReadRef& InResetValueIn,
			const FEnumCurveFunctionReadRef& InCurveFunction,
			const FFloatReadRef& InExponentIn,
			const EMetaSoundFadeOutputType InOutputType)
			: TriggerIn(InTriggerIn)
			, ResetIn(InResetIn)
			, DurationIn(InDurationIn)
			, StartValueIn(InStartValueIn)
			, EndValueIn(InEndValueIn)
			, StartTimeIn(InStartTimeIn)
			, ResetValueIn(InResetValueIn)
			, CurveFunctionIn(InCurveFunction)
			, ExponentIn(InExponentIn)
			, OutputType(InOutputType)
			, OnTrigger(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnDone(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, OnNearlyDone(FTriggerWriteRef::CreateNew(InParams.OperatorSettings))
			, FloatFadeValue(TDataWriteReferenceFactory<float>::CreateExplicitArgs(InParams.OperatorSettings))
			, AudioFadeValue(TDataWriteReferenceFactory<FAudioBuffer>::CreateExplicitArgs(InParams.OperatorSettings))
		{
			// Reset Environment Parameters
			NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();

			if (OutputType == EMetaSoundFadeOutputType::AudioBufferType)
			{
				SampleRate = FMath::Max(1.f, InParams.OperatorSettings.GetSampleRate());
			}
			else if (OutputType == EMetaSoundFadeOutputType::FloatType)
			{
				SampleRate = FMath::Max(1.f, InParams.OperatorSettings.GetActualBlockRate());
			}

			ActualFramesPerBlock = SampleRate / InParams.OperatorSettings.GetActualBlockRate();

			// Reset outputs
			OnTrigger->Reset();
			OnDone->Reset();
			OnNearlyDone->Reset();

			*FloatFadeValue = 0.f;
			AudioFadeValue->Zero();
		}

		virtual ~TFadeNodeOperator() override = default;

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace FadeNodePrivate::FadeVertexNames;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputTrigger), TriggerIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputReset), ResetIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDuration), DurationIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartValue), StartValueIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputEndValue), EndValueIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputCurveFunction), CurveFunctionIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputExponent), ExponentIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStartTime), StartTimeIn);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputResetValue), ResetValueIn);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace FadeNodePrivate::FadeVertexNames;

			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnTrigger), OnTrigger);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnDone), OnDone);
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputOnNearlyDone), OnNearlyDone);

			if (OutputType == EMetaSoundFadeOutputType::FloatType)
			{
				InOutVertexData.BindWriteVertex(MakeOutputVertexName(OutputType), FloatFadeValue);
			}
			else if (OutputType == EMetaSoundFadeOutputType::AudioBufferType)
			{
				InOutVertexData.BindWriteVertex(MakeOutputVertexName(OutputType), AudioFadeValue);
			}

		}

		// Reset Operating Environment
		void Reset(const IOperator::FResetParams& InParams)
		{
			// Reset Environment Parameters
			NumFramesPerBlock = InParams.OperatorSettings.GetNumFramesPerBlock();

			if (OutputType == EMetaSoundFadeOutputType::AudioBufferType)
			{
				SampleRate = FMath::Max(1.f, InParams.OperatorSettings.GetSampleRate());
			}
			else if (OutputType == EMetaSoundFadeOutputType::FloatType)
			{
				SampleRate = FMath::Max(1.f, InParams.OperatorSettings.GetActualBlockRate());
			}

			ActualFramesPerBlock = SampleRate / InParams.OperatorSettings.GetActualBlockRate();

			// Reset outputs
			OnTrigger->Reset();
			OnDone->Reset();
			OnNearlyDone->Reset();

			*FloatFadeValue = 0.f;
			AudioFadeValue->Zero();
		}

		void Execute()
		{
			using namespace FadeNodePrivate;

			OnTrigger->AdvanceBlock();
			OnNearlyDone->AdvanceBlock();
			OnDone->AdvanceBlock();

			bool bApplyUpdateCrossfade = Duration != *DurationIn ||
				StartValue != *StartValueIn ||
				EndValue != *EndValueIn ||
				StartTime != *StartTimeIn ||
				bResetValue != *ResetValueIn ||
				CurveFunction != *CurveFunctionIn ||
				Exponent != *ExponentIn ||
				ResetIn->IsTriggeredInBlock();

			// check for any updates to input params
			if (bApplyUpdateCrossfade)
			{
				// Update FadeOutBuffer
				FadeOutBuffer.SetNumZeroed(ActualFramesPerBlock);

				// Update FadeInBuffer
				FadeInBuffer.SetNumZeroed(ActualFramesPerBlock);

				// Generate segment before updating
				Fade.GenerateSegment(FadeOutBuffer.GetData(), FadeOutBuffer.Num(), false);

				// Apply fade
				Audio::ArrayFade(FadeOutBuffer, 1.f, 0.f);

				// Generate Fade In interpolation buffer
				Audio::ArraySetToConstantInplace(FadeInBuffer, 1.f);
				Audio::ArrayFade(FadeInBuffer, 0.f, 1.f);

				if (ResetIn->IsTriggeredInBlock())
				{
					// Reset Fade
					InternalReset();
				}
				else
				{
					// Update parameters last
					UpdateParams();
				}
			}

			// Setup fade block
			TArray<float> FadeValue;
			FadeValue.SetNumZeroed(ActualFramesPerBlock);
			TArrayView<float> FadeValueView(FadeValue);

			TriggerIn->ExecuteBlock(
				// OnPreTrigger
				[&](int32 StartFrame, int32 EndFrame)
				{
					check(StartFrame >= 0);

					// Determine maximum slice size
					const int32 MaxSliceSize = FMath::Clamp(EndFrame - StartFrame, 0, ActualFramesPerBlock);

					// Determine best slice start frame
					const int32 SliceStartFrame = FMath::Clamp(StartFrame, 0, ActualFramesPerBlock - 1);

					// Get on pre trigger frame view
					TArrayView<float> FadeValueViewPreTrigger = FadeValueView.Slice(SliceStartFrame, MaxSliceSize);

					// If fade is active, check for doneness
					if (Fade.IsActive())
					{
						// Get OnDone Frames
						const int32 OnDoneFrame = Fade.DoneWithinInFrameRange(MaxSliceSize);
						const int32 OnNearlyDoneFrame = Fade.DoneWithinInFrameRange(MaxSliceSize + ActualFramesPerBlock);

						// If frame completes within this block, trigger on done frame
						if (OnDoneFrame >= 0)
						{
							OnDone->TriggerFrame(OnDoneFrame);
						}
						// If frame completes within the next two blocks, trigger on done frame
						else if (OnNearlyDoneFrame >= MaxSliceSize)
						{
							OnNearlyDone->TriggerFrame(OnNearlyDoneFrame - MaxSliceSize);
						}
					}

					// Generate fade segment
					Fade.GenerateSegment(FadeValueViewPreTrigger.GetData(), FadeValueViewPreTrigger.Num());

					// If parameters updated this frame, execute crossfade
					if (bApplyUpdateCrossfade)
					{
						// Setup on pre trigger Fade Out frame slice
						TArrayView<float> FadeOutBufferView(FadeOutBuffer);
						TArrayView<float> CrossfadeBufferViewSlice = FadeOutBufferView.Slice(SliceStartFrame, MaxSliceSize);

						// Setup on pre trigger Fade In frame slice
						TArrayView<float> FadeInBufferView(FadeInBuffer);
						TArrayView<float> FadeInBufferViewSlice = FadeInBufferView.Slice(SliceStartFrame, MaxSliceSize);

						// Apply fade in to value buffer
						Audio::ArrayMultiplyInPlace(FadeInBufferViewSlice, FadeValueViewPreTrigger);

						// Mix two buffers together for crossfade
						Audio::ArrayMixIn(CrossfadeBufferViewSlice, FadeValueViewPreTrigger);
					}
				},
				// OnTrigger
				[&](int32 StartFrame, int32 EndFrame)
				{
					check(StartFrame >= 0);

					// Pass through On Trigger 
					OnTrigger->TriggerFrame(StartFrame);

					// Get maximum slice size for this on trigger block
					const int32 MaxSliceSize = FMath::Clamp(EndFrame - StartFrame, 0, ActualFramesPerBlock);

					// Determine best slice start frame
					const int32 SliceStartFrame = FMath::Clamp(StartFrame, 0, ActualFramesPerBlock - 1);

					// Determine start sample if seeking into the middle of the fade playback
					const int32 StartSample = FMath::Floor(StartTime.GetSeconds() * static_cast<float>(SampleRate));

					// Pass in which sample to start fade playback on
					Fade.Start(StartSample);

					// Create an on trigger slice of the value buffer
					TArrayView<float> FadeValueViewOnTrigger = FadeValueView.Slice(SliceStartFrame, MaxSliceSize);

					// Get OnDone Frames
					const int32 OnDoneFrame = Fade.DoneWithinInFrameRange(MaxSliceSize);
					const int32 OnNearlyDoneFrame = Fade.DoneWithinInFrameRange(MaxSliceSize + ActualFramesPerBlock);

					// If frame completes within this block, trigger on done frame
					if (OnDoneFrame >= 0)
					{
						OnDone->TriggerFrame(OnDoneFrame);
					}
					// If frame completes within the next two blocks, trigger on done frame
					else if (OnNearlyDoneFrame >= 0)
					{
						OnNearlyDone->TriggerFrame(OnNearlyDoneFrame - MaxSliceSize);
					}

					// Generate the fade segment for this on trigger part of the block
					Fade.GenerateSegment(FadeValueViewOnTrigger.GetData(), FadeValueViewOnTrigger.Num());

					// Determine if you need to update parameter values, implement crossfade
					if (bApplyUpdateCrossfade)
					{

						// Setup on on trigger Fade Out frame slice
						TArrayView<float> FadeOutBufferView(FadeOutBuffer);
						TArrayView<float> CrossfadeBufferViewSlice = FadeOutBufferView.Slice(SliceStartFrame, MaxSliceSize);
						
						// Setup on on trigger Fade In frame slice
						TArrayView<float> FadeInBufferView(FadeInBuffer);
						TArrayView<float> FadeInBufferViewSlice = FadeInBufferView.Slice(SliceStartFrame, MaxSliceSize);

						// Apply fade in to value buffer
						Audio::ArrayMultiplyInPlace(FadeInBufferViewSlice, FadeValueViewOnTrigger);

						// Mix two buffers together for crossfade
						Audio::ArrayMixIn(CrossfadeBufferViewSlice, FadeValueViewOnTrigger);
					}
				}
			);

			// Check if first index is valid
			if (FadeValue.IsValidIndex(0))
			{
				// If the Output Type is Float, pass buffer index 0 to float output
				if (OutputType == EMetaSoundFadeOutputType::FloatType)
				{
					*FloatFadeValue = FadeValue[0];
				}
				// If the Output Type is Audio Buffer, copy buffer over
				else
				{
					memcpy(AudioFadeValue->GetData(), FadeValue.GetData(), FadeValue.GetAllocatedSize());
				}
			}
		}

	private:

		// Reset Fade
		void InternalReset()
		{
			using namespace FadeNodePrivate;

			// Cache and guard against bad input
			CacheParams();

			// calculate number of samples in fade
			const float TotalFadeSamples = FMath::Max(0.f, Duration.GetSeconds()) * SampleRate;

			// Construct fade data
			FFadeData Data =
			{
				TotalFadeSamples,
				StartValue,
				EndValue,
				Exponent,
				CurveFunction,
				bResetValue
			};

			// Initialize Fade with Fade Data
			Fade.Init(Data);
		}

		void CacheParams()
		{
			// Update cached input values
			Duration = FTime::FromSeconds(FMath::Clamp(DurationIn->GetSeconds(), 0.f, 10000.f));
			StartValue = *StartValueIn;
			EndValue = *EndValueIn;
			StartTime = FTime::FromSeconds(FMath::Clamp(StartTimeIn->GetSeconds(), 0.f, Duration.GetSeconds()));
			bResetValue = *ResetValueIn;
			CurveFunction = *CurveFunctionIn;
			Exponent = FMath::Max(*ExponentIn, 0.f);
		}

		void UpdateParams()
		{
			using namespace FadeNodePrivate;

			// Cache and guard against bad input
			CacheParams();

			// Cacluate latest total number of fade samples
			const float TotalFadeSamples = FMath::Max(0.f, Duration.GetSeconds()) * SampleRate;

			// Construct fade data
			FFadeData Data =
			{
				TotalFadeSamples,
				StartValue,
				EndValue,
				Exponent,
				CurveFunction,
				bResetValue
			};

			// Update the fade data
			Fade.SetFadeData(Data);
		}

		// Node Input Data
		FTriggerReadRef	TriggerIn;
		FTriggerReadRef	ResetIn;
		FTimeReadRef DurationIn;
		FFloatReadRef StartValueIn;
		FFloatReadRef EndValueIn;
		FTimeReadRef StartTimeIn;
		FBoolReadRef ResetValueIn;
		FEnumCurveFunctionReadRef CurveFunctionIn;
		FFloatReadRef ExponentIn;

		EMetaSoundFadeOutputType OutputType = EMetaSoundFadeOutputType::FloatType;

		// Cached IO Ref
		FTime Duration = FTime(0.f);
		float StartValue = 0.f;
		float EndValue = 1.f;
		FadeNodePrivate::ECurveFunction CurveFunction = FadeNodePrivate::ECurveFunction::EqualPower;
		float Exponent = 1.f;
		FTime StartTime = FTime(0.f);
		bool bResetValue = false;

		// Node Output Data
		FTriggerWriteRef OnTrigger;
		FTriggerWriteRef OnDone;
		FTriggerWriteRef OnNearlyDone;

		FFloatWriteRef FloatFadeValue;
		FAudioBufferWriteRef AudioFadeValue;

		// Class Data

		TArray<float> FadeOutBuffer;
		TArray<float>FadeInBuffer;

		// This will either be the block rate or sample rate depending on if this is block-rate or audio-rate envelope
		float SampleRate = 0.0f;
		int32 NumFramesPerBlock = 0;
		int32 ActualFramesPerBlock = 480;

		// Fade Generator
		FadeNodePrivate::FFade Fade;
	};

	/** TFadeNode
	*
	*  Creates a Simple Fade node.
	*/
	using FMetaSoundFadeNode = TNodeFacade<TFadeNodeOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FMetaSoundFadeNode, FMetaSoundFadeNodeConfiguration);
}

FMetaSoundFadeNodeConfiguration::FMetaSoundFadeNodeConfiguration()
	: OutputType(EMetaSoundFadeOutputType::FloatType)
{ }

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundFadeNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound::FadeNodePrivate;

	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(GetVertexInterface(OutputType)));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundFadeNodeConfiguration::GetOperatorData() const
{
	using namespace Metasound::FadeNodePrivate::FadeVertexNames;

	return MakeShared<Metasound::FFadeNodeOperatorData>(OutputType);
}

#undef LOCTEXT_NAMESPACE