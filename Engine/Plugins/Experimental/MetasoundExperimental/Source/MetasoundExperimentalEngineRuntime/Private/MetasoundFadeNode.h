// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "Containers/EnumAsByte.h"
#include "MetasoundOperatorData.h"
#include "MetasoundFrontendDocument.h"

#include "MetasoundFadeNode.generated.h"

UENUM()
enum EMetaSoundFadeOutputType : uint8
{
	FloatType UMETA(DisplayName = "Float"),
	AudioBufferType UMETA(DisplayName = "Audio")
};

USTRUCT()
struct FMetaSoundFadeNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundFadeNodeConfiguration();

	// Output Type for Fade Value
	UPROPERTY(EditAnywhere, Category = General)
	TEnumAsByte<EMetaSoundFadeOutputType> OutputType = EMetaSoundFadeOutputType::FloatType;

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};
