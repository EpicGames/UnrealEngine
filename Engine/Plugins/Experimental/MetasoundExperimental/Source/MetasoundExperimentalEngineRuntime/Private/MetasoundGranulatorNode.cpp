// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundGranulatorNode.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/GrainEnvelope.h"
#include "DSP/SampleRateConverter.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundTrigger.h"
#include "MetasoundVertex.h"
#include "MetasoundWave.h"
#include "DSP/Vbap.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProxyPlayer.h"
#include "TypeFamily/ChannelTypeFamily.h"
#include "MetasoundCatPannerNode.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalNodes_GranulatorNode"

namespace Metasound
{
	class FChannelAgnosticType;
	struct FBuildOperatorParams;
	class FNodeFacade;

	namespace GranulatorPrivate
	{
		METASOUND_PARAM(InputSpawnGrain,	"Grain Spawn",				"Spawns a new grain from the wave asset.");
		METASOUND_PARAM(InputStop,			"Stop",						"Stops and resets the granulator state.");
		METASOUND_PARAM(InputWaveAsset,		"Wave Asset",				"Input sound wave to granulate.");
		METASOUND_PARAM(InputLocationX,		"Grain Location X",			"Location in time (as a normalized fraction of audio file length) to spawn new grains from.");
		METASOUND_PARAM(InputChannelY,		"Grain Channel Y",			"Location in channel-space (as a normalized fraction of audio file channel count) to generated blended grains.");
		METASOUND_PARAM(InputDuration,		"Grain Duration",			"Duration (in milliseconds) of new spawned grains.");
		METASOUND_PARAM(InputPlaybackRate, "Grain Playback Rate",		"Playback rate of new grains (1.0 is normal speed, -1.0 is backward).");
		METASOUND_PARAM(InputVolume,		"Grain Volume",				"Linear volume scale of new grains.");
		METASOUND_PARAM(InputPan,			"Grain Pan",				"Azimuthal pan of new grains (0.0 = left, 1.0 = right, 0.5 = center, 1.5 is behind).");
		METASOUND_PARAM(OutputToCat,		"Output",	 				"Output audio.");

		// Linkage.
		const FLazyName FGranulatorOperatorData::OperatorDataTypeName = TEXT("FGranulatorOperatorData");

		struct FDecodedDataChunk
		{
			// The start frame of the decoded audio chunk
			int32 FrameStart = INDEX_NONE;

			// The actual decoded audio. 
			// Size is NumFrames * NumChannels * MetaSoundSampleRate (after SRC the source file)
			Audio::FAlignedFloatBuffer PCMAudio;

			// The number of audio frames in the data chunk
			int32 NumFramesInChunk = 0;

			// Count of the number of grains actively using this chunk
			int32 NumGrainsUsingChunk = 0;
		};

		// Data used for spawning new grains
		struct FSpawnGrainData
		{
			int32 StartFrameOffset = 0;
			float LocationTimeSeconds = 0.0f;
			float LocationChannelFraction = 0.5f;
			int32 DurationFrames = 100;
			float PlaybackRate = 1.0f;
			float Volume = 1.0f;
			float Pan = 0.5;
			EGranularEnvelope Envelope = EGranularEnvelope::Gaussian;
		};

		// Active grain data for a single grain
		struct FGrain
		{
			// The frame this grain starts rendering in the current block. Allows grains to schedule mid-block at the top of the render block.
			int32 FrameStartOffset = 0;
			
			// The number of frames this grain has rendered
			// If this is larger than GrainDurationFrames it's an inactive grain
			// this is used to look up the grain envelope to find the amplitude of the grain per frame
			int32 CurrentRenderedFramesCount = INDEX_NONE;

			// Index into the decoded data array that this grain is currently using
			int32 DecodedDataChunkIndex = 0;

			// The current read frame of the grain. Float to support dynamic playback rate.
			float CurrentReadFrame = INDEX_NONE;
			float CurrentPlaybackRate = 1.0f;

			// Previous sample values (from previous decoded chunk) for the Y-channel reading feature
			float PrevASampleValue = 0.0f;
			float PrevBSampleValue = 0.0f;

			// The duration of this grain. Set when the grain spawns. 
			int32 GrainDurationFrames = 0;

			// The current channel position of the grain (interpolated across the audio channels)
			float CurrentLocationChannelFraction = 0.5f;

			// Grain panning information
			float CurrentPanParameter = 0.5f;
			float LastUpdatePanningFraction = -1.0f;
			float LastUpdatePrevChannelGain = -1.0f;
			float LastUpdateNextChannelGain = -1.0f;

			// Grain volume information
			float CurrentVolume = 1.0f;
			float PreviousVolume = 1.0f;

			EGranularEnvelope EnvelopType = EGranularEnvelope::Gaussian;
		};
	}

	static const Audio::FDiscreteChannelTypeFamily& GetDiscreteFamilyChecked(const Audio::FChannelTypeFamily& FamilyType)
	{
		check(FamilyType.GetFamilyName() == Audio::FDiscreteChannelTypeFamily::GetFamilyTypeName());
		return static_cast<const Audio::FDiscreteChannelTypeFamily&>(FamilyType);
	}

	class FGranulatorOperator final : public TExecutableOperator<FGranulatorOperator>
	{
	public:
		using FDiscreteChannelTypeFamily = Audio::FDiscreteChannelTypeFamily;
		using FGranulatorOperatorData = GranulatorPrivate::FGranulatorOperatorData;
		using FDecodedDataChunk = GranulatorPrivate::FDecodedDataChunk;
		using FGrain = GranulatorPrivate::FGrain;
		using FSpawnGrainData = GranulatorPrivate::FSpawnGrainData;

		FGranulatorOperator(
				const TSharedPtr<const FGranulatorOperatorData>& InOperatorData,
				const FBuildOperatorParams& InParams, 
				FTriggerReadRef&& InSpawnGrain,
				FTriggerReadRef&& InStop,
				FWaveAssetReadRef&& InWaveAsset,
				FFloatReadRef&& InGrainLocationX,
				FFloatReadRef&& InGrainChannelY,
				FFloatReadRef&& InGrainDuration,
				FFloatReadRef&& InGrainPlaybackRate,
				FFloatReadRef&& InGrainVolume,
				FFloatReadRef&& InGrainPan,
				FDiscreteChannelAgnosticTypeWriteRef&& InOutputAudio
				)
			: OperatorData(InOperatorData)
			, SpawnGrainTrigger(MoveTemp(InSpawnGrain))
			, StopTrigger(MoveTemp(InStop))
			, WaveAsset(MoveTemp(InWaveAsset))
			, GrainLocationX(MoveTemp(InGrainLocationX))
			, GrainChannelY(MoveTemp(InGrainChannelY))
			, GrainDuration(MoveTemp(InGrainDuration))
			, GrainPlaybackRate(MoveTemp(InGrainPlaybackRate))
			, GrainVolume(MoveTemp(InGrainVolume))
			, GrainPan(MoveTemp(InGrainPan))
			, OutputAudio(MoveTemp(InOutputAudio))
			, Settings(InParams.OperatorSettings)
			, DiscreteFamily(GetDiscreteFamilyChecked(OutputAudio->GetType()))
			, SRC(Audio::ISampleRateConverter::CreateSampleRateConverter())
			, GranularEnvelopeType(InOperatorData->GranularEnvelope)
			, Panner(OutputAudio->GetType().GetPanner())
		{
			// Generate the grain envelope data for the different grain envelopes
			int32 NumEnvelopeTypes = (int32)EGranularEnvelope::Hann + 1;
			GrainEnvelopeData.AddDefaulted(NumEnvelopeTypes);
			for (int32 i = 0; i < NumEnvelopeTypes; ++i)
			{
				Audio::Grain::EEnvelope EnvType = (Audio::Grain::EEnvelope)i;
				Audio::Grain::GenerateEnvelopeData(GrainEnvelopeData[i], 256, EnvType);
			}
		}
		virtual ~FGranulatorOperator() override = default;

		static const FVertexInterface MakeInterface(const FName InOutputFormat)
		{
			using namespace GranulatorPrivate;
			// inputs
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FWaveAsset>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputWaveAsset)));
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputSpawnGrain)));
			InputInterface.Add(TInputDataVertex<FTrigger>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputStop)));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputLocationX), 0.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputChannelY), 0.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputDuration), 100.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPlaybackRate), 1.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputVolume), 1.0f));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputPan), 0.5f));

			// outputs
			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(FOutputDataVertex(METASOUND_GET_PARAM_NAME(OutputToCat), InOutputFormat, METASOUND_GET_PARAM_METADATA(OutputToCat), EVertexAccessType::Reference));
			
			return FVertexInterface(InputInterface, OutputInterface);
		}
			
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace GranulatorPrivate;

			const FGranulatorOperatorData* ConfigData = CastOperatorData<const FGranulatorOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!ConfigData)
			{
				return MakeUnique<FNoOpOperator>();
			}
			const FName AudioOutputToName = ConfigData->OutputAudioTypeName;
			const Audio::FChannelTypeFamily* ConcretePanToType = Audio::GetChannelRegistry().FindConcreteChannel(AudioOutputToName);
			if (!ConcretePanToType)
			{
				return MakeUnique<FNoOpOperator>();
			}

			const TSharedPtr<const FGranulatorOperatorData>& OperatorDataSharedPtr = StaticCastSharedPtr<const FGranulatorOperatorData>(InParams.Node.GetOperatorData());

			TDataReadReference<FTrigger> InputSpawnGrainRef = InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputSpawnGrain), InParams.OperatorSettings);
			TDataReadReference<FTrigger> InputStopRef = InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputStop), InParams.OperatorSettings);
			TDataReadReference<FWaveAsset> InputWaveAssetRef = InParams.InputData.GetOrCreateDefaultDataReadReference<FWaveAsset>(METASOUND_GET_PARAM_NAME(InputWaveAsset), InParams.OperatorSettings);
			TDataReadReference<float> InputLocationXRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputLocationX), InParams.OperatorSettings);
			TDataReadReference<float> InputChannelYRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputChannelY), InParams.OperatorSettings);
			TDataReadReference<float> InputDurationRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputDuration), InParams.OperatorSettings);
			TDataReadReference<float> InputPlaybackRateRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPlaybackRate), InParams.OperatorSettings);
			TDataReadReference<float> InputVolumeRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputVolume), InParams.OperatorSettings);
			TDataReadReference<float> InputPanRef = InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputPan), InParams.OperatorSettings);


			// Make Discrete CAT output.
			// These are defined in data so we must use "DerivedAs" 
			TDataWriteReference<FDiscreteChannelAgnosticType> CatPin =
				FDiscreteChannelAgnosticTypeWriteRef::CreateNewDerivedAs<FDiscreteChannelAgnosticType>(
					ConcretePanToType->GetName(),
					Metasound::GetMetasoundDataTypeId<FDiscreteChannelAgnosticType>(),
					InParams.OperatorSettings,
					ConcretePanToType->GetName()
				);
			
			return MakeUnique<FGranulatorOperator>(
				OperatorDataSharedPtr,
				InParams,
				MoveTemp(InputSpawnGrainRef),
				MoveTemp(InputStopRef),
				MoveTemp(InputWaveAssetRef),
				MoveTemp(InputLocationXRef),
				MoveTemp(InputChannelYRef),
				MoveTemp(InputDurationRef),
				MoveTemp(InputPlaybackRateRef),
				MoveTemp(InputVolumeRef),
				MoveTemp(InputPanRef),
				MoveTemp(CatPin)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace GranulatorPrivate;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputSpawnGrain), SpawnGrainTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputStop), StopTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputWaveAsset), WaveAsset);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputLocationX), GrainLocationX);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputChannelY), GrainChannelY);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputDuration), GrainDuration);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPlaybackRate), GrainPlaybackRate);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputVolume), GrainVolume);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputPan), GrainPan);
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace GranulatorPrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputToCat), OutputAudio);
		}

		void Reset(const FResetParams& InParams)
		{
			Execute();
		}

		bool Initialize()
		{
			check(!bPlaying);

 			CurrentWaveAsset = *WaveAsset;
			const TSharedPtr<const FSoundWaveProxy> Proxy = CurrentWaveAsset.GetWaveProxy();

			if (Proxy)
			{
				SoundWaveDataPtr = Proxy->GetSoundWaveDataRef();
			}
			else
			{
				SoundWaveDataPtr.Reset();
				return false;
			}

			SourceFileSampleRate = SoundWaveDataPtr->GetSampleRate();
			SourceFileDurationSeconds = SoundWaveDataPtr->GetDuration();
			SourceFileNumFrames = SoundWaveDataPtr->GetNumFrames();
			SourceFileNumChannels = SoundWaveDataPtr->GetNumChannels();

			FSoundWaveProxyPlayer::FSettings ProxyPlayerSettings(SourceFileSampleRate, SourceFileNumChannels);
			ProxyPlayerSettings.MaxDecodeSizeFrames = DecodedAudioSizeInSeconds * SourceFileSampleRate;
			SoundWaveProxyPlayerPtr = FSoundWaveProxyPlayer::Create(ProxyPlayerSettings);
			SoundWaveProxyPlayerPtr->SetSoundWave(SoundWaveDataPtr.ToSharedRef());
			SoundWaveProxyPlayerPtr->SetLoop(true);

			DecodedChunks.Reset();
			DecodedChunks.AddDefaulted(3);

			// Scratch buffer to write the audio generated from mono-grains before applying panning
			MonoGrainBuffer.Reset();
			MonoGrainBuffer.AddZeroed(Settings.GetNumFramesPerBlock());

			return true;
		}

		void SpawnGrain(const FSpawnGrainData& InSpawnGrainData)
		{
			// Try to retrieve a decoded data chunk for the current read frame based on the curret, interpolated playhead time seconds value
			int32 CurrentReadFrame = InSpawnGrainData.LocationTimeSeconds * Settings.GetSampleRate();
			int32 DecodedDataChunkIndex = GetDecodedDataChunkIndexForCurrentReadIndex(CurrentReadFrame);
			if (DecodedDataChunkIndex == INDEX_NONE)
			{
				DecodedDataChunkIndex = DecodeDataChunkIndexForCurrentReadIndex(CurrentReadFrame);
			}
			check(DecodedDataChunkIndex != INDEX_NONE);

			// Get the grain runtime data for the new grain spawn
			FGrain NewGrain;
			NewGrain.FrameStartOffset = InSpawnGrainData.StartFrameOffset;
			NewGrain.CurrentRenderedFramesCount = 0;
			NewGrain.DecodedDataChunkIndex = DecodedDataChunkIndex;
			NewGrain.CurrentReadFrame = (float)CurrentReadFrame;
			NewGrain.CurrentPlaybackRate = InSpawnGrainData.PlaybackRate;
			NewGrain.GrainDurationFrames = InSpawnGrainData.DurationFrames;
			NewGrain.CurrentLocationChannelFraction = InSpawnGrainData.LocationChannelFraction;
			NewGrain.CurrentPanParameter = InSpawnGrainData.Pan;
			NewGrain.CurrentVolume = InSpawnGrainData.Volume;
			NewGrain.EnvelopType = GranularEnvelopeType;

			ActiveGrains.Add(NewGrain);

			DecodedChunks[DecodedDataChunkIndex].NumGrainsUsingChunk++;
			GrainCount++;
		}

		void DecodeToDataChunk(FDecodedDataChunk& InOutDataChunk, float InDecoderSeekTimeSeconds)
		{
			check(InOutDataChunk.NumGrainsUsingChunk == 0);
			check(DecodedAudioSizeInSeconds > 0.0f);
			check(SourceFileSampleRate > 0.0f);
			check(InDecoderSeekTimeSeconds >= 0.0f);
			check(SourceFileNumChannels > 0);
			check(SoundWaveDataPtr.IsValid());

			check(SoundWaveProxyPlayerPtr.IsValid());
			float DecoderSeekTimeSeconds = FMath::Max(InDecoderSeekTimeSeconds - 0.5f * DecodedAudioSizeInSeconds, 0.0f);
			// If proxy player already exists, then simply seek the decoder to the desired location
			SoundWaveProxyPlayerPtr->SeekToTime(DecoderSeekTimeSeconds);

			float AudioMixerSampleRate = Settings.GetSampleRate();

			InOutDataChunk.FrameStart = DecoderSeekTimeSeconds * AudioMixerSampleRate;

			// We allocate the size of buffer pre-SRC based on source file sample rate
			int32 DecodedAudioSize = DecodedAudioSizeInSeconds * SourceFileSampleRate * SourceFileNumChannels;
			check(DecodedAudioSize > 0);
			InOutDataChunk.PCMAudio.Reset();
			InOutDataChunk.PCMAudio.AddUninitialized(DecodedAudioSize);

			// This does the actual decoding of the audio to match the size of the input buffer
			SoundWaveProxyPlayerPtr->GenerateSourceAudio(InOutDataChunk.PCMAudio);

			// Check if we need to do SRC. If we do, this will change the allocated 
			// size (potentially expanding or shrinking) to match 
			// the audio mixer sample rate.
			if (!FMath::IsNearlyEqual(SourceFileSampleRate, AudioMixerSampleRate))
			{
				SRC->Init((float)SourceFileSampleRate / AudioMixerSampleRate, SourceFileNumChannels);
				TArray<float> SampleRateConvertedPCM;
				SRC->ProcessFullbuffer(InOutDataChunk.PCMAudio.GetData(), InOutDataChunk.PCMAudio.Num(), SampleRateConvertedPCM);
				InOutDataChunk.PCMAudio = MoveTemp(SampleRateConvertedPCM);
			}

			InOutDataChunk.NumFramesInChunk = InOutDataChunk.PCMAudio.Num() / SourceFileNumChannels;
		}

		int32 GetDecodedDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex)
		{
			for (int32 i = 0; i < DecodedChunks.Num(); ++i)
			{
				const FDecodedDataChunk& DecodedChunk = DecodedChunks[i];
				if (DecodedChunk.PCMAudio.Num() > 0)
				{
					int32 DecodedFrameCount = DecodedChunk.NumFramesInChunk;
					if (InReadFrameIndex >= DecodedChunk.FrameStart && InReadFrameIndex < DecodedChunk.FrameStart + DecodedFrameCount)
					{
						// We found a decoded audio chunk that contains the desired read frame index
						return i;
					}
				}
			}
			return INDEX_NONE;
		}

		int32 DecodeDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex)
		{
			float AudioMixerSampleRate = Settings.GetSampleRate();

			// No decoded chunk was found for desired read frame index. 
			// This indicates that we need to decode more audio.
			for (int32 i = 0; i < DecodedChunks.Num(); ++i)
			{
				FDecodedDataChunk& DecodedChunk = DecodedChunks[i];
				if (!DecodedChunk.NumGrainsUsingChunk)
				{
					float DecoderSeekTimeSeconds = InReadFrameIndex / AudioMixerSampleRate;
					DecodeToDataChunk(DecodedChunk, DecoderSeekTimeSeconds);
					check(DecodedChunk.PCMAudio.Num() > 0);
					return i;
				}
			}

			FDecodedDataChunk NewChunk;
			float DecoderSeekTimeSeconds = InReadFrameIndex / AudioMixerSampleRate;
			DecodeToDataChunk(NewChunk, DecoderSeekTimeSeconds);
			check(NewChunk.PCMAudio.Num() > 0);
			DecodedChunks.Add(MoveTemp(NewChunk));
			return DecodedChunks.Num() - 1;
		}

		
		void GetOutputSpeakerChannelData(FGrain& InGrain, float& OutPanningFraction, Audio::IDiscretePanner::FPanResult& OutPrev, Audio::IDiscretePanner::FPanResult& OutNext)
		{ 
			check(InGrain.CurrentPanParameter >= 0.0f && InGrain.CurrentPanParameter < 2.0f);
			const float AzimuthDegrees = NormalizedAzimuthToDegrees(InGrain.CurrentPanParameter);

			if (Panner)
			{
				Audio::IDiscretePanner::FOutputParams OutParams;
				Panner->ComputeGains( { .AzimuthDegrees = AzimuthDegrees, .ElevationDegrees = 0, .bAllowAzimuthMirroring = true }, OutParams);
				OutPrev = OutParams.Results[0];
				OutNext = OutParams.Results[1];
			}
			
#if 0
			float AzimuthDegrees = FMath::Fmod((InGrain.CurrentPanParameter * 180.0f) + 270.0f, 360.0f);
			int32 NumOutputChannels = ChannelAzimuthData.Num();
			for (int32 i = 0; i < NumOutputChannels; ++i)
			{
				const FAzimuthalSpeakerData& PrevData = ChannelAzimuthData[i];
				const FAzimuthalSpeakerData& NextData = ChannelAzimuthData[(i + 1) % NumOutputChannels];

				float AzimuthDelta = FMath::Fmod(NextData.Azimuth - PrevData.Azimuth + 360.0f, 360.0f);
				float RelAzimuth = FMath::Fmod(AzimuthDegrees - PrevData.Azimuth + 360.0f, 360.0f);

				if (RelAzimuth >= 0.0f && RelAzimuth < AzimuthDelta)
				{
					float PanningFraction = AzimuthDelta > KINDA_SMALL_NUMBER ? (RelAzimuth / AzimuthDelta) : 0.0f;
					OutPrev = PrevData;
					OutNext = NextData;
					OutPanningFraction = PanningFraction;
					return;
				}
			}
#endif //0			
		}
		

		//void ComputePanGains(float InPanningFraction, float& OutPrevGain, float& OutNextGain) const
		//{
		//	check(InPanningFraction >= 0.0f && InPanningFraction <= 1.0f);
		//
		//FMath::SinCos(&OutNextGain, &OutPrevGain, 0.5f * PI * InPanningFraction);
		//}

		void MixMonoGrainToOutput(TArrayView<float>& InMonoBufferView, FGrain& InGrain)
		{
			//FAzimuthalSpeakerData PrevData;
			//FAzimuthalSpeakerData NextData;
			Audio::IDiscretePanner::FPanResult PrevData;
			Audio::IDiscretePanner::FPanResult NextData;
			float PanningFraction = 0.0f;
			GetOutputSpeakerChannelData(InGrain, PanningFraction, PrevData, NextData);

			if (PrevData.ChannelID.IsNone() || NextData.ChannelID.IsNone())
			{
				return;
			}
			const int32 PrevSpeakerIndex = DiscreteFamily.FindSpeakerIndex(PrevData.ChannelID);
			check(PrevSpeakerIndex != INDEX_NONE);

			const int32 NextSpeakerIndex = DiscreteFamily.FindSpeakerIndex(NextData.ChannelID);
			check(NextSpeakerIndex != INDEX_NONE);

			TArrayView<float> PrevChannelOutAudioBufferView = OutputAudio->GetChannel(PrevSpeakerIndex);
			TArrayView<float> NextChannelOutAudioBufferView = OutputAudio->GetChannel(NextSpeakerIndex);

			// We use optimized panning algorithm that interpolates the panning amount through the block
			float StartingPanningFraction = InGrain.LastUpdatePanningFraction < 0.0f ? PanningFraction : InGrain.LastUpdatePanningFraction;
			float EndingPanningFraction = PanningFraction;

			// This is the first execute so we haven't yet cached our prev/next channel pans
			//if (InGrain.LastUpdatePanningFraction < 0.0f)
			//{
			//	ComputePanGains(StartingPanningFraction, InGrain.LastUpdatePrevChannelGain, InGrain.LastUpdateNextChannelGain);
			//}

			float CurrentPrevChannelGain = PrevData.Gain;
			float CurrentNextChannelGain = NextData.Gain;
			//ComputePanGains(EndingPanningFraction, CurrentPrevChannelGain, CurrentNextChannelGain);

			Audio::ArrayMixIn(InMonoBufferView, PrevChannelOutAudioBufferView, InGrain.LastUpdatePrevChannelGain, CurrentPrevChannelGain);
			Audio::ArrayMixIn(InMonoBufferView, NextChannelOutAudioBufferView, InGrain.LastUpdateNextChannelGain, CurrentNextChannelGain);

			// lerp through the buffer to the target panning amount
			InGrain.LastUpdatePanningFraction = PanningFraction;
			InGrain.LastUpdatePrevChannelGain = CurrentPrevChannelGain;
			InGrain.LastUpdateNextChannelGain = CurrentNextChannelGain;
		}

		void GetNewDecodedData(FGrain& InGrain, int32 InFrameOffset)
		{
			// We're no longer using this decoded audio chunk
			DecodedChunks[InGrain.DecodedDataChunkIndex].NumGrainsUsingChunk--;
			check(DecodedChunks[InGrain.DecodedDataChunkIndex].NumGrainsUsingChunk >= 0);

			// If we're calling this function, that means that we ran out of data for the current read frame chunk, so load up the chunk for the next frame
			InGrain.DecodedDataChunkIndex = GetDecodedDataChunkIndexForCurrentReadIndex((int32)InGrain.CurrentReadFrame + InFrameOffset);
			if (InGrain.DecodedDataChunkIndex == INDEX_NONE)
			{
				InGrain.DecodedDataChunkIndex = DecodeDataChunkIndexForCurrentReadIndex((int32)InGrain.CurrentReadFrame + InFrameOffset);
			}
			check(InGrain.DecodedDataChunkIndex != INDEX_NONE);

			DecodedChunks[InGrain.DecodedDataChunkIndex].NumGrainsUsingChunk++;
		}

		void RenderActiveGrains(int32 InNumFramesToRender)
		{
			check(SourceFileNumChannels > 0);

			// Reverse iterate so we can quickly remove grains when they're done
			for (int32 GrainIndex = ActiveGrains.Num() - 1; GrainIndex >= 0; --GrainIndex)
			{
				FGrain& Grain = ActiveGrains[GrainIndex];

				// Cache the grain envelope ref for this grain
				Audio::Grain::FEnvelope& GrainEnv = GrainEnvelopeData[(int32)Grain.EnvelopType];

				// This is a frame offset into the current block to account for grains starting mid-block
				int32 GrainWriteIndex = Grain.FrameStartOffset;
				int32 NumFramesToRenderForBlock = InNumFramesToRender - GrainWriteIndex;

				// After this block render, we won't need to use this frame offset
				// It is only ever non-zero for newly spawned grains
				Grain.FrameStartOffset = 0;

				// This is the number of frames we have left to render for this grain
				int32 NumFramesRemainingInGrain = Grain.GrainDurationFrames - Grain.CurrentRenderedFramesCount;

				// We only need to render frames for either how many are in this block or how many are left in the grain to render
				NumFramesToRenderForBlock = FMath::Min(NumFramesToRenderForBlock, NumFramesRemainingInGrain);
				check(NumFramesToRenderForBlock > 0);

				// For inner-loop audio rendering, we avoid the constant array-access checks that happen on our containers
				FMemory::Memzero(MonoGrainBuffer.GetData(), MonoGrainBuffer.Num() * sizeof(float));
				float* MonoGrainBufferPtr = MonoGrainBuffer.GetData();

				// Calculate which 2 channels we need blend for the output
				float ChannelIndex = Grain.CurrentLocationChannelFraction * (SourceFileNumChannels - 1);
				int32 ChannelAIndex = (int32)ChannelIndex;
				int32 ChannelBIndex = (ChannelAIndex + 1) % SourceFileNumChannels;
				// The alpha blend between channels A and B
				float ChannelAlpha = 1.0f - (ChannelIndex - (float)ChannelAIndex);

				bool bGrainFinished = false;
				while (NumFramesToRenderForBlock > 0 && !bGrainFinished)
				{
					check(Grain.DecodedDataChunkIndex != INDEX_NONE);

					// Make sure we have a valid decode data chunk ready for rendering
					const FDecodedDataChunk* DecodedData = &DecodedChunks[Grain.DecodedDataChunkIndex];

					if (Grain.CurrentPlaybackRate > 0.0f)
					{
						// The number of frames that this grain is offset from the decoded data chunk
						// We are rounding up to nearest integer since float-frame read indices require us to read to the next frame to interpolate.
						// We should be storing the previous audio data so we can't discard the old chunk if our "next" frame read is off this current chunk
						
						int32 NumFramesOffsetInDecodedData = (int32)(Grain.CurrentReadFrame) - DecodedData->FrameStart + 1;

						// These are the total number of frames left in the current chunk we can consume
						// Note this can be negative if the pitch scale is > 1.0
						int32 NumFramesPossibleToConsumeInCurrentChunk = DecodedData->NumFramesInChunk - NumFramesOffsetInDecodedData;

						// If we've totally consumed this decoded audio chunk, we need to get a new decoded audio chunk
						if (NumFramesPossibleToConsumeInCurrentChunk <= 0)
						{
							GetNewDecodedData(Grain, 1);

							DecodedData = &DecodedChunks[Grain.DecodedDataChunkIndex];
						}
					}
					else
					{
						if (Grain.CurrentReadFrame < (float)DecodedData->FrameStart)
						{
							// Wrap the read frame around the audio file
							if (Grain.CurrentReadFrame < 0.0f)
							{
								Grain.CurrentReadFrame += (float)SourceFileNumFrames;
							}

							GetNewDecodedData(Grain, -1);

							DecodedData = &DecodedChunks[Grain.DecodedDataChunkIndex];
							check(Grain.CurrentReadFrame >= (float)DecodedData->FrameStart);
						}
					}

					int32 FrameWriteIndex = 0;

					while (NumFramesToRenderForBlock > 0 && !bGrainFinished)
					{
						float EnvelopeFraction = FMath::Clamp((float)Grain.CurrentRenderedFramesCount / Grain.GrainDurationFrames, 0.0f, 1.0f);
						float GrainAmplitude = Audio::Grain::GetValue(GrainEnv, EnvelopeFraction);
							
						// Need to interpolate from the previoux block
						float DecodedSampleDataPrev = 0.0f;
						float DecodedSampleDataNext = 0.0f;
						float ReadFrameAlpha = 0;

						float ReadFrameOffset = Grain.CurrentReadFrame - (float)DecodedData->FrameStart;

						// Playing forward
						if (Grain.CurrentPlaybackRate > 0.0f)
						{
							// We're still reading from the previous block of decoded audio (which is behind our current chunk)
							if (ReadFrameOffset < 0.0f)
							{
								check(ReadFrameOffset >= -1.0f);
								DecodedSampleDataPrev = Grain.PrevASampleValue;
								if (SourceFileNumChannels == 1)
								{
									DecodedSampleDataNext = DecodedData->PCMAudio[0];
								}
								else
								{
									float SampleA = DecodedData->PCMAudio[ChannelAIndex];
									float SampleB = DecodedData->PCMAudio[ChannelBIndex];
									DecodedSampleDataNext = FMath::Lerp(SampleA, SampleB, ChannelAlpha);
								}
								ReadFrameAlpha = 1.0f + ReadFrameOffset;
								check(ReadFrameAlpha >= 0.0f && ReadFrameAlpha <= 1.0f);
							}
							else
							{
								int32 PrevDataReadIndex = (int32)ReadFrameOffset;
								int32 NextDataReadIndex = PrevDataReadIndex + 1;

								// We've consumed all the data from this audio chunk
								if (NextDataReadIndex >= DecodedData->NumFramesInChunk)
								{
									break;
								}

								ReadFrameAlpha = ReadFrameOffset - (float)PrevDataReadIndex;

								if (SourceFileNumChannels == 1)
								{
									DecodedSampleDataPrev = DecodedData->PCMAudio[PrevDataReadIndex];
									DecodedSampleDataNext = DecodedData->PCMAudio[NextDataReadIndex];
								}
								else
								{
									int32 SampleReadIndex = PrevDataReadIndex * SourceFileNumChannels;
									float SampleA = DecodedData->PCMAudio[SampleReadIndex + ChannelAIndex];
									float SampleB = DecodedData->PCMAudio[SampleReadIndex + ChannelBIndex];

									DecodedSampleDataPrev = FMath::Lerp(SampleA, SampleB, ChannelAlpha);

									SampleReadIndex = NextDataReadIndex * SourceFileNumChannels;
									SampleA = DecodedData->PCMAudio[SampleReadIndex + ChannelAIndex];
									SampleB = DecodedData->PCMAudio[SampleReadIndex + ChannelBIndex];

									DecodedSampleDataNext = FMath::Lerp(SampleA, SampleB, ChannelAlpha);
								}
							}
						}
						// Playing backward!
						else
						{
							// We're still reading from the previous block of decoded audio data (which is in front our current chunk)
							if (((int32)ReadFrameOffset + 1) >= DecodedData->NumFramesInChunk)
							{
								// We should only be 1 frame into the next block
								DecodedSampleDataPrev = Grain.PrevASampleValue;
								if (SourceFileNumChannels == 1)
								{
									DecodedSampleDataNext = DecodedData->PCMAudio.Last();
								}
								else
								{									
									int32 SampleReadIndex = (DecodedData->NumFramesInChunk - 1) * SourceFileNumChannels;
									float SampleA = DecodedData->PCMAudio[SampleReadIndex + ChannelAIndex];
									float SampleB = DecodedData->PCMAudio[SampleReadIndex + ChannelBIndex];
									DecodedSampleDataNext = FMath::Lerp(SampleA, SampleB, ChannelAlpha);
								}

								float Delta = FMath::Clamp((float)DecodedData->NumFramesInChunk - ReadFrameOffset, 0.0f, 1.0f);
								ReadFrameAlpha = 1.0f - Delta;
							}
							else
							{
								int32 NextDataReadIndex = (int32)ReadFrameOffset;
								int32 PrevDataReadIndex = NextDataReadIndex + 1;

								ReadFrameAlpha = (float)PrevDataReadIndex - ReadFrameOffset;

								if (SourceFileNumChannels == 1)
								{
									DecodedSampleDataPrev = DecodedData->PCMAudio[PrevDataReadIndex];
									DecodedSampleDataNext = DecodedData->PCMAudio[NextDataReadIndex];
								}
								else
								{
									int32 SampleReadIndex = PrevDataReadIndex * SourceFileNumChannels;
									float SampleA = DecodedData->PCMAudio[SampleReadIndex + ChannelAIndex];
									float SampleB = DecodedData->PCMAudio[SampleReadIndex + ChannelBIndex];

									DecodedSampleDataPrev = FMath::Lerp(SampleA, SampleB, ChannelAlpha);

									SampleReadIndex = NextDataReadIndex * SourceFileNumChannels;
									SampleA = DecodedData->PCMAudio[SampleReadIndex + ChannelAIndex];
									SampleB = DecodedData->PCMAudio[SampleReadIndex + ChannelBIndex];

									DecodedSampleDataNext = FMath::Lerp(SampleA, SampleB, ChannelAlpha);
								}
							}
						}

						// Store it here for interpolating between buffers
						Grain.PrevASampleValue = DecodedSampleDataPrev;

						float SampleValue = FMath::Lerp(DecodedSampleDataPrev, DecodedSampleDataNext, ReadFrameAlpha);

						SampleValue *= GrainAmplitude;

						// Write the value to the output buffer
						MonoGrainBuffer[GrainWriteIndex++] = SampleValue;

						Grain.CurrentRenderedFramesCount++;
						NumFramesToRenderForBlock--;

						Grain.CurrentReadFrame += Grain.CurrentPlaybackRate;
						bGrainFinished = (Grain.CurrentRenderedFramesCount >= Grain.GrainDurationFrames);

						// We're reading backward in the audio file and we've read off the chunk
						if (!bGrainFinished && Grain.CurrentReadFrame < (float)DecodedData->FrameStart)
						{
							break;
						}
					}
				}

				// We have our generated mono grain buffer, now write it to the outputs
				// This will do the spatialization of the grain
				TArrayView<float> MonoGrainBufferView = TArrayView<float>(MonoGrainBuffer);

				// Apply the grain volume scale on the mono grain buffer before panning
				if (Grain.PreviousVolume < 0.0f)
				{
					Grain.PreviousVolume = Grain.CurrentVolume;
				}

				if (FMath::IsNearlyEqual(Grain.PreviousVolume, Grain.CurrentVolume))
				{
					Audio::ArrayMultiplyByConstantInPlace(MonoGrainBufferView, Grain.CurrentVolume);
				}
				else
				{
					Audio::ArrayFade(MonoGrainBufferView, Grain.PreviousVolume, Grain.CurrentVolume);
					Grain.PreviousVolume = Grain.CurrentVolume;
				}

				MixMonoGrainToOutput(MonoGrainBufferView, Grain);

				// If the grain has finished, then remove it from the active grain list
				if (bGrainFinished)
				{
					DecodedChunks[Grain.DecodedDataChunkIndex].NumGrainsUsingChunk--;
					check(DecodedChunks[Grain.DecodedDataChunkIndex].NumGrainsUsingChunk >= 0);
					ActiveGrains.RemoveAtSwap(GrainIndex, EAllowShrinking::No);
				}
			}
		}

		void Execute()
		{
			// Zero everytime.
			OutputAudio->Zero();
			
			const TArray<int32>& SpawnTriggerFrames = SpawnGrainTrigger->GetTriggeredFrames();
			const TArray<int32>& StopTriggerFrames = StopTrigger->GetTriggeredFrames();

			int32 LastStartTriggerFrame = SpawnTriggerFrames.Num() > 0 ? SpawnTriggerFrames.Last() : INDEX_NONE;
			int32 LastStopTriggerFrame = StopTriggerFrames.Num() > 0 ? StopTriggerFrames.Last() : INDEX_NONE;

			// Check if we're stopping this block

			// Initialize the granulator if we're not currently playing
			if (!bPlaying && LastStartTriggerFrame >= 0)
			{
				// Initialize things if we didn't trigger stop this block or the last stop trigger is before the last start trigger
				if (LastStopTriggerFrame == INDEX_NONE || LastStopTriggerFrame < LastStartTriggerFrame)
				{
					bPlaying = Initialize();
				}
			}
	
			// We're not playing so just early return
			if (!bPlaying)
			{
				return;
			}

			// Check if we're spawning new grains this block
			if (SpawnTriggerFrames.Num() > 0)
			{
				// Wrap the grain location param to be between 0.0 and 1.0
				float WrappedLocation = FMath::Fmod(*GrainLocationX, 1.0f);
				if (WrappedLocation < 0.0f)
				{
					WrappedLocation += 1.0f;
				}				
				float WrappedSeekTimeSeconds = WrappedLocation * SourceFileDurationSeconds;
				
				float ClampedGrainDurationSeconds = 0.001f * FMath::Clamp(*GrainDuration, 0.0f, 2000.0f);
				int32 ClampedGrainDurationFrames = (int32) (ClampedGrainDurationSeconds * Settings.GetSampleRate());
				ClampedGrainDurationFrames = FMath::Max(ClampedGrainDurationFrames, 50);
				
				float ClampedPlaybackRate = *GrainPlaybackRate;
				if (FMath::IsNearlyZero(ClampedPlaybackRate))
				{
					ClampedPlaybackRate = KINDA_SMALL_NUMBER;
				}

				float ClampedVolume = FMath::Clamp(*GrainVolume, -4.0f, 4.0f);

				float WrappedPan = FMath::Fmod(*GrainPan, 2.0f);
				if (WrappedPan < 0.0f)
				{
					WrappedPan += 2.0f;
				}

				float ClampedChannelY = FMath::Clamp(*GrainChannelY, 0.0f, 1.0f);

				GranularEnvelopeType = OperatorData->GranularEnvelope;

				FSpawnGrainData NewGrainSpawnData;
				NewGrainSpawnData.LocationTimeSeconds = WrappedSeekTimeSeconds;
				NewGrainSpawnData.LocationChannelFraction = ClampedChannelY;
				NewGrainSpawnData.DurationFrames = ClampedGrainDurationFrames;
				NewGrainSpawnData.PlaybackRate = ClampedPlaybackRate;
				NewGrainSpawnData.Volume = ClampedVolume;
				NewGrainSpawnData.Pan = WrappedPan;
				NewGrainSpawnData.Envelope = GranularEnvelopeType;

				// Only spawn new grains up until the first Stop-Trigger this frame
				int32 FirstStopTrigger = StopTriggerFrames.Num() > 0 ? StopTriggerFrames[0] : Settings.GetNumFramesPerBlock();
				for (int32 i = 0; i < SpawnTriggerFrames.Num(); ++i)
				{
					if (SpawnTriggerFrames[i] >= FirstStopTrigger)
					{
						break;
					}

					NewGrainSpawnData.StartFrameOffset = SpawnTriggerFrames[i];
					SpawnGrain(NewGrainSpawnData);
				}
			}

			// Zero the output of the channel agnostic audio
			OutputAudio->Zero();

			RenderActiveGrains(Settings.GetNumFramesPerBlock());
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace GranulatorPrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "GranulatorOperator", "" },
				1, // Major version
				0, // Minor version
				LOCTEXT("GranulatorNodeName", "Granulator"),	
				LOCTEXT("GranulatorNodeNameDescription", "A Node that granulates an input sound wave asset."),	
				TEXT("UE"), // Author
				LOCTEXT("ExampleGranulatorPromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				MakeInterface(TEXT("Cat:Stereo1Dot0")),
				{}
			};
		}
	private:
		TSharedPtr<const FGranulatorOperatorData> OperatorData;
		FTriggerReadRef SpawnGrainTrigger;
		FTriggerReadRef StopTrigger;
		FWaveAssetReadRef WaveAsset;
		FFloatReadRef GrainLocationX;
		FFloatReadRef GrainChannelY;
		FFloatReadRef GrainDuration;
		FFloatReadRef GrainPlaybackRate;
		FFloatReadRef GrainVolume;
		FFloatReadRef GrainPan;
		FDiscreteChannelAgnosticTypeWriteRef OutputAudio;
		TArray<float> MonoGrainBuffer;

		FOperatorSettings Settings;
		const FDiscreteChannelTypeFamily& DiscreteFamily;
		
		// Place to store decoded chunks. We need 2 chunks for actively playing grains plus a free chunk.
		TArray<FDecodedDataChunk> DecodedChunks;

		// The grain data used to render the granular audio
		TArray<FGrain> ActiveGrains;

		bool bPlaying = false;

		// How much audio to decode per decode block
		float DecodedAudioSizeInSeconds = 0.25f;

		float SourceFileSampleRate = 0.0f;
		float SourceFileDurationSeconds = 0.0f;
		int32 SourceFileNumFrames = 0;
		int32 SourceFileNumChannels = 1;

		int32 GrainCount = 0;

		FWaveAsset CurrentWaveAsset;
		TSharedPtr<const FSoundWaveData> SoundWaveDataPtr;
		TUniquePtr<FSoundWaveProxyPlayer> SoundWaveProxyPlayerPtr;
		TUniquePtr<Audio::ISampleRateConverter> SRC;
		EGranularEnvelope GranularEnvelopeType = EGranularEnvelope::Hann;
		TArray<Audio::Grain::FEnvelope> GrainEnvelopeData;
		const Audio::IDiscretePanner* Panner = nullptr;
	}; 
		
	using FGranulatorNode = TNodeFacade<FGranulatorOperator>;

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FGranulatorNode, FMetaSoundGranulatorNodeConfiguration);
} // namespace Metasound

TArray<FPropertyTextFName> UMetaSoundGranulatorNodeOptionsHelper::GetSoundFileFormatChannelOptions()
{
	// These are output from Unreal.
	// Hardcoded for now.
	static const TArray<FName> InternalFormats {
		TEXT("Cat:Mono1Dot0"),
		TEXT("Cat:Stereo2Dot0"),
		TEXT("Cat:Surround5Dot1"),
		TEXT("Cat:Surround7Dot1"),
	};

	const TArray<TSharedRef<const Audio::FChannelTypeFamily>> AllFormats = Audio::GetChannelRegistry().GetAllChannelFormats();
	TArray<FPropertyTextFName> FormatsOptions;
	for (const TSharedRef<const Audio::FChannelTypeFamily>& Format : AllFormats)
	{
		if (const Audio::FDiscreteChannelTypeFamily* Discrete = Format->Cast<Audio::FDiscreteChannelTypeFamily>())
		{
			if (InternalFormats.Contains(Discrete->GetName()))
			{
				FormatsOptions.Emplace(Format->GetName(), FText::FromString(Format->GetFriendlyName()));
			}
		}
	}
	return FormatsOptions;
}

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundGranulatorNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(Metasound::FGranulatorOperator::MakeInterface(this->OutputAudioTypeName)));
}

FMetaSoundGranulatorNodeConfiguration::FMetaSoundGranulatorNodeConfiguration()
	: OperatorData(MakeShared<Metasound::GranulatorPrivate::FGranulatorOperatorData>(OutputAudioTypeName, GranularEnvelope))
{
}


TSharedPtr<const Metasound::IOperatorData> FMetaSoundGranulatorNodeConfiguration::GetOperatorData() const
{
	OperatorData->OutputAudioTypeName = OutputAudioTypeName;
	OperatorData->GranularEnvelope = GranularEnvelope;
	return OperatorData;
}

#undef LOCTEXT_NAMESPACE // 
