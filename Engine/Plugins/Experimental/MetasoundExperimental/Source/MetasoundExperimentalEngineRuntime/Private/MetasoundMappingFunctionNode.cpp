// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundMappingFunctionNode.h" 

#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

#include "MetasoundDataFactory.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundMappingFunctionNodeRuntime"

namespace Metasound::Experimental
{
	namespace MappingFunctionNodePrivate
	{
		METASOUND_PARAM(InputValue, "Input", "Input float value");
		METASOUND_PARAM(InMin, "In Min", "Input Min Value");
		METASOUND_PARAM(InMax, "In Max", "Input Max Value");
		METASOUND_PARAM(InOutMin, "Out Min", "Output Min Value");
		METASOUND_PARAM(InOutMax, "Out Max", "Output Max Value");

		METASOUND_PARAM(OutputFloat, "Output", "Mapped float output");

		FVertexInterface GetVertexInterface()
		{
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InputValue), 0.0f});
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InMin), 0.0f});
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InMax), 1.0f});
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InOutMin), 0.0f});
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InOutMax), 1.0f});

			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(TOutputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(OutputFloat)});

			return FVertexInterface
			{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
	}

	const FLazyName FMappingFunctionNodeOperatorData::OperatorDataTypeName = "MappingFunctionOperatorData";

	class FMappingFunctionNodeOperator : public TExecutableOperator<FMappingFunctionNodeOperator>
	{
	public:
		FMappingFunctionNodeOperator(const TSharedPtr<const FMappingFunctionNodeOperatorData>& InOperatorData, const FBuildOperatorParams& InParams)
			: OperatorData(InOperatorData)
			, InMin(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MappingFunctionNodePrivate::InMinName, InParams.OperatorSettings))
			, InMax(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MappingFunctionNodePrivate::InMaxName, InParams.OperatorSettings))
			, InOutMin(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MappingFunctionNodePrivate::InOutMinName, InParams.OperatorSettings))
			, InOutMax(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MappingFunctionNodePrivate::InOutMaxName, InParams.OperatorSettings))
			, InputValue(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MappingFunctionNodePrivate::InputValueName, InParams.OperatorSettings))
			, OutputValue(TDataWriteReferenceFactory<float>::CreateExplicitArgs(InParams.OperatorSettings))
		{
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MappingFunctionNodePrivate;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputValue), InputValue);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InMin), InMin);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InMax), InMax);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InOutMin), InOutMin);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InOutMax), InOutMax);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace MappingFunctionNodePrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputFloat), OutputValue);
		}

		void Execute()
		{
			if (!OperatorData.IsValid())
			{
				return;
			}

			const FRuntimeFloatCurve& Curve = OperatorData->MappingFunction;
			const FRichCurve& RichCurve = *Curve.GetRichCurveConst();
			if (!RichCurve.GetNumKeys())
			{
				return;
			}

			float MinFuncTime = 0.0f;
			float MaxFuncTime = 0.0f;
			float MinFuncValue = 0.0f;
			float MaxFuncValue = 0.0f;

			RichCurve.GetTimeRange(MinFuncTime, MaxFuncTime);
			RichCurve.GetValueRange(MinFuncValue, MaxFuncValue);

			float TimeInputVal = 0.0f;
			if (OperatorData->bWrapInputs)
			{
				TimeInputVal = FMath::Wrap(*InputValue, *InMin, *InMax);
			}
			else
			{
				TimeInputVal = FMath::Clamp(*InputValue, *InMin, *InMax);
			}

			float TimeInputMapped = FMath::GetMappedRangeValueClamped(FVector2D{ *InMin, *InMax}, FVector2D{ MinFuncTime, MaxFuncTime }, TimeInputVal);
			float OutputVal = RichCurve.Eval(TimeInputMapped);
			float OutputValMapped = FMath::GetMappedRangeValueClamped(FVector2D{ MinFuncValue, MaxFuncValue }, FVector2D{ *InOutMin, *InOutMax}, OutputVal);
			
			*OutputValue = OutputValMapped;
		}
		
		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace MappingFunctionNodePrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "MappingFunctionOperator", "" },
				1, // Major version
				0, // Minor version
				LOCTEXT("MappingFunctionNodeName", "Mapping Function"),
				LOCTEXT("MappingFunctionNodeDescription", "A node which maps inputs to outputs using a curve mapping function."),
				TEXT("UE"), // Author
				LOCTEXT("MappingFunctionPromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				GetVertexInterface(),
				{}
			};
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace MappingFunctionNodePrivate;

			if (const FMappingFunctionNodeOperatorData* OperatorData = CastOperatorData<const FMappingFunctionNodeOperatorData>(InParams.Node.GetOperatorData().Get()))
			{
				// Need to get a shared ptr of the operator data so we can get live updates from the interface
				const TSharedPtr<const FMappingFunctionNodeOperatorData>& OperatorDataSharedPtr = StaticCastSharedPtr<const FMappingFunctionNodeOperatorData>(InParams.Node.GetOperatorData());
				return MakeUnique<FMappingFunctionNodeOperator>(OperatorDataSharedPtr, InParams);
			}

			return nullptr;
		}

	private:
		// Contains configured float data
		TSharedPtr<const FMappingFunctionNodeOperatorData> OperatorData;

		TDataReadReference<float> InMin;
		TDataReadReference<float> InMax;
		TDataReadReference<float> InOutMin;
		TDataReadReference<float> InOutMax;

		TDataReadReference<float> InputValue;
		TDataWriteReference<float> OutputValue;
	};

	using FMappingFunctionNode = TNodeFacade<FMappingFunctionNodeOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FMappingFunctionNode, FMetaSoundMappingFunctionNodeConfiguration);
}

FMetaSoundMappingFunctionNodeConfiguration::FMetaSoundMappingFunctionNodeConfiguration()
	: OperatorData(MakeShared<Metasound::Experimental::FMappingFunctionNodeOperatorData>(MappingFunction, bWrapInputs))
{
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundMappingFunctionNodeConfiguration::GetOperatorData() const
{
	OperatorData->MappingFunction = MappingFunction;
	OperatorData->bWrapInputs = bWrapInputs;
	return OperatorData;
}

#undef LOCTEXT_NAMESPACE // MetasoundMappingFunctionNodeRuntime