// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

#include "MetasoundDataFactory.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundTimerNodeRuntime"

namespace Metasound::Experimental
{
	namespace TimerNodePrivate
	{
		METASOUND_PARAM(InStart, "Start", "Starts the Timer output from 0.0.");
		METASOUND_PARAM(InStop, "Stop", "Stops the Timer output");
		METASOUND_PARAM(InScale, "Scale", "Scales the output time of the Timer");

		METASOUND_PARAM(OutputTime, "Timer Output", "Time in seconds since Timer started.");

		FVertexInterface GetVertexInterface()
		{
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FTrigger>{METASOUND_GET_PARAM_NAME_AND_METADATA(InStart)});
			InputInterface.Add(TInputDataVertex<FTrigger>{METASOUND_GET_PARAM_NAME_AND_METADATA(InStop)});
			InputInterface.Add(TInputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(InScale), 1.0f});

			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(TOutputDataVertex<float>{METASOUND_GET_PARAM_NAME_AND_METADATA(OutputTime)});

			return FVertexInterface
			{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
	}

	class FTimerNodeOperator : public TExecutableOperator<FTimerNodeOperator>
	{
	public:
		FTimerNodeOperator(const FBuildOperatorParams& InParams)
			: InStartTrigger(InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(TimerNodePrivate::InStartName, InParams.OperatorSettings))
			, InStopTrigger(InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(TimerNodePrivate::InStopName, InParams.OperatorSettings))
			, InScale(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(TimerNodePrivate::InScaleName, InParams.OperatorSettings))
			, OutputTime(TDataWriteReferenceFactory<float>::CreateExplicitArgs(InParams.OperatorSettings))
			, FramesPerSecond(InParams.OperatorSettings.GetSampleRate())
			, FramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
		{
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TimerNodePrivate;

			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InStart), InStartTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InStop), InStopTrigger);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InScale), InScale);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace TimerNodePrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputTime), OutputTime);
		}

		void Execute()
		{
			float ClampedScale = *InScale;
			if (FMath::IsNearlyZero(*InScale, UE_KINDA_SMALL_NUMBER))
			{
				ClampedScale = UE_KINDA_SMALL_NUMBER;
			}
			ScaledFramesPerSecond = FramesPerSecond / ClampedScale;
			check(ScaledFramesPerSecond > 0.0f || ScaledFramesPerSecond < 0.0f);
			*OutputTime = (float)FramesSinceStart / ScaledFramesPerSecond;

			// In a given block, if we start and stop a bunch of times, we only want to look at the last start and stops
			const TArray<int32>& StartTriggerFrames = InStartTrigger->GetTriggeredFrames();
			const TArray<int32>& StopTriggerFrames = InStopTrigger->GetTriggeredFrames();

			int32 LastStartTriggerFrame = StartTriggerFrames.Num() > 0 ? StartTriggerFrames.Last() : INDEX_NONE;
			int32 LastStopTriggerFrame = StopTriggerFrames.Num() > 0 ? StopTriggerFrames.Last() : INDEX_NONE;

			// Start and stop were both triggered in this block
			if (LastStartTriggerFrame != INDEX_NONE && LastStopTriggerFrame != INDEX_NONE)
			{
				// If we start then stopped as the last triggers, then our "time" value is the delta between these frames
				if (LastStopTriggerFrame > LastStartTriggerFrame)
				{
					// We track the number of frames since start as this time delta so next block it can be reported in output
					FramesSinceStart = LastStopTriggerFrame - LastStartTriggerFrame;
					bIsTimerRunning = false;
				}
				else
				{
					// We track the remaining frames to the end of the block 
					FramesSinceStart = FramesPerBlock - LastStartTriggerFrame;
					bIsTimerRunning = true;
				}
			}
			// Only stop was triggered, so accumulate up to the stop trigger frame
			else if (LastStopTriggerFrame != INDEX_NONE)
			{
				FramesSinceStart += LastStopTriggerFrame;
				bIsTimerRunning = false;
			}
			// Only start was triggered, so track from the start frame to the end of the block
			else if (LastStartTriggerFrame != INDEX_NONE)
			{
				FramesSinceStart = FramesPerBlock - LastStartTriggerFrame;
				bIsTimerRunning = true;
			}
			// No stop or start trigger was made, so simply accumulate the time if we're running
			// otherwise, don't need to do anything.
			else if (bIsTimerRunning)
			{
				FramesSinceStart += FramesPerBlock;
			}
		}
		
		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace TimerNodePrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "TimerNodeOperator", "" },
				1, // Major version
				0, // Minor version
				LOCTEXT("TimerNodeName", "Timer"),
				LOCTEXT("TimerNodeDescription", "A node which outputs a time value in seconds with optional scaling."),
				TEXT("UE"), // Author
				LOCTEXT("TimernodePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				GetVertexInterface(),
				{}
			};
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			return MakeUnique<FTimerNodeOperator>(InParams);
		}

	private:
		TDataReadReference<FTrigger> InStartTrigger;
		TDataReadReference<FTrigger> InStopTrigger;
		TDataReadReference<float> InScale;

		TDataWriteReference<float> OutputTime;

		float FramesPerSecond = 0.0f;
		float ScaledFramesPerSecond = 0.0f;
		int32 FramesSinceStart = 0;
		int32 FramesPerBlock = 0;
		bool bIsTimerRunning = false;
	};

	using FTimerNode = TNodeFacade<FTimerNodeOperator>;
	METASOUND_REGISTER_NODE(FTimerNode);
}

#undef LOCTEXT_NAMESPACE // MetasoundTimerNodeRuntime