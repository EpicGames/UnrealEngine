// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"
#include "Templates/SharedPointer.h"

#include <atomic>

namespace Audio
{
	/**
	 * Thread-safe, lock-free "view" over proxy data published from the game thread and
	 * consumed from the audio thread.
	 *
	 * Modeled as a one-directional linked list: the head is the "latest" version and
	 * older nodes hold an atomic pointer to their successor so stale references can
	 * always resolve to the current head via GetLatest(). Pushing a new version is a
	 * compare-exchange on the tail's NextPtr, after which NextSharedPtr is assigned
	 * (the caller's shared-ref keeps the new head alive during the transition).
	 *
	 * Typical usage:
	 *
	 *   struct FMyData { ... };
	 *   class FMyProxy : public Audio::TCatProxyView<FMyProxy, FMyData>
	 *   {
	 *   public:
	 *       IMPL_AUDIOPROXY_CLASS(FMyProxy);
	 *   };
	 *
	 *   // Game thread: create + publish.
	 *   Proxy = FMyProxy::Create(MoveTemp(InitialData));
	 *   Proxy = Proxy->New(MoveTemp(UpdatedData));
	 *
	 *   // Audio thread:
	 *   const TSharedRef<const FMyProxy> Latest = Proxy->GetLatest();
	 *
	 * Trivial scalar updates can be applied directly on GetData() without pushing a
	 * new node; only structural changes (resizing arrays, etc.) require New().
	 *
	 * NOTE: scratch-duplicated into MetasoundExperimentalEngineRuntime for the CAT
	 * Wave Player 2.0 move. MetasoundPolyphonyInternal retains its own copy pending
	 * consolidation (see cat-wave-player-move-spec.md §0).
	 *
	 * Threading contract:
	 *   - New() must be called from a single thread. In practice is called from 
	 *     the game thread only. Callers must hold a shared ref to the head they 
	 *     are publishing from for the duration of the call.
	 *   - GetLatest() / IsLatest() / GetData() are safe to call concurrently from
	 *     reader threads (including the audio render thread). Readers acquire-load
	 *     NextPtr so the NextSharedPtr write in New() happens-before the read.
	 *   - The owning object (here, UCatSoundWaveContainer) must outlive all audio-
	 *     thread readers. This is the actual contract: the Wave Player operator caches
	 *     a TSharedPtr<const TCatProxyView> on the audio thread and drops it in Reset().
	 *     The chain itself is kept alive by the publisher-held head ref + each node's
	 *     NextSharedPtr; no reader-thread destructor coordination is required.
	 */
	template<typename TType, typename TData>
	class TCatProxyView : public IProxyData, public TSharedFromThis<TCatProxyView<TType, TData>>
	{
	protected:
		static constexpr bool bWasAudioProxyClassImplemented = false;

		enum EPrivateToken {};

		std::atomic<TCatProxyView*> NextPtr = nullptr;
		TSharedPtr<TCatProxyView> NextSharedPtr = nullptr;

		TData Data;

	public:
		TCatProxyView(EPrivateToken)
			: IProxyData(TType::GetAudioProxyTypeName())
		{
			static_assert(TType::bWasAudioProxyClassImplemented, "Include IMPL_AUDIOPROXY_CLASS in your TCatProxyView subclass.");
		}

		TCatProxyView(EPrivateToken, TData&& InData)
			: IProxyData(TType::GetAudioProxyTypeName())
			, Data(MoveTemp(InData))
		{
			static_assert(TType::bWasAudioProxyClassImplemented, "Include IMPL_AUDIOPROXY_CLASS in your TCatProxyView subclass.");
		}

		virtual ~TCatProxyView() override
		{
			// Contract: the owning object (UCatSoundWaveContainer) must outlive any
			// audio-thread reader holding a shared ref to a node in this chain. That
			// is how lifetime is actually guaranteed today; this destructor is a
			// no-op safety net because the atomic store + shared ptr reset are no-
			// longer-relevant after the last ref is dropped.
			NextPtr.store(nullptr, std::memory_order_relaxed);
			NextSharedPtr.Reset();
		}

		// Create() instantiates the base TCatProxyView (not TType) and static-casts the
		// shared ref to TType*. The actual runtime object is therefore TCatProxyView, and
		// any QueryInterface override declared on TType is never reached via the vtable.
		// We answer TType::GetAudioProxyTypeName() here so CheckTypeCast<TType>() and
		// GetAs<TType>() both work against the real object.
		virtual void* QueryInterface(const FName InterfaceId) override
		{
			if (InterfaceId == TType::GetAudioProxyTypeName())
			{
				return this;
			}
			return IProxyData::QueryInterface(InterfaceId);
		}

		/** Create a fresh proxy reference; takes ownership of InData. */
		static TSharedRef<TType> Create(TData&& InData)
		{
			return MakeShared<TType>(EPrivateToken(), MoveTemp(InData));
		}

		/**
		 * Publish a new data version; returns the new head. The caller should replace
		 * its cached shared ref with the return value so subsequent New() calls append
		 * to the linear chain.
		 *
		 * Memory ordering: we assign NextSharedPtr BEFORE the release-store that
		 * publishes NewPtr via CAS. A reader which acquire-loads a non-null NextPtr
		 * is then guaranteed to observe a fully-constructed NextSharedPtr, which
		 * keeps the chain's lifetime intact on the reader side. Pairs with the
		 * acquire load in GetLatest() / IsLatest().
		 *
		 * Single-writer contract: the audio proxy chain is published from the game
		 * thread only. Concurrent publishers are not supported (NextSharedPtr is a
		 * non-atomic TSharedPtr). Readers may be on any thread, including the audio
		 * render thread.
		 */
		TSharedRef<TType> New(TData&& InData)
		{
			TSharedRef<TType> NewProxy = MakeShared<TType>(EPrivateToken(), MoveTemp(InData));
			TCatProxyView* NewPtr = &NewProxy.Get();
			TCatProxyView* LatestPtr = this;
			// Walk to the current tail; under the single-writer contract we are the
			// only thread performing this walk/CAS.
			while (TCatProxyView* NextRaw = LatestPtr->NextPtr.load(std::memory_order_acquire))
			{
				LatestPtr = NextRaw;
			}
			// Establish the shared-ref write BEFORE the release-store that makes
			// NewPtr observable to readers. Acquire-loading readers that see NewPtr
			// in NextPtr are then guaranteed to observe the NextSharedPtr write.
			LatestPtr->NextSharedPtr = NewProxy.ToSharedPtr();
			TCatProxyView* NullPtr = nullptr;
			const bool bPublished = LatestPtr->NextPtr.compare_exchange_strong(
				NullPtr, NewPtr,
				std::memory_order_release, std::memory_order_acquire);
			check(bPublished); // single-writer contract
			return NewProxy;
		}

		bool IsLatest() const
		{
			return NextPtr.load(std::memory_order_acquire) == nullptr;
		}

		TSharedRef<const TType> GetLatest() const
		{
			const TCatProxyView* LatestPtr = this;
			while (TCatProxyView* NextRaw = LatestPtr->NextPtr.load(std::memory_order_acquire))
			{
				LatestPtr = NextRaw;
			}
			return StaticCastSharedRef<const TType>(LatestPtr->AsShared());
		}

		const TData& GetData() const { return Data; }
		TData& GetData() { return Data; }
	};
}
