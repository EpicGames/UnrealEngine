// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CatAudioProxyView.h"
#include "Containers/Array.h"
#include "IAudioProxyInitializer.h"
#include "Sound/ISoundWaveContainer.h"
#include "Sound/SoundWave.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "CatSoundWaveContainer.generated.h"

class USoundWave;
class FSoundWaveProxy;
class FCatSoundWaveContainerProxy;
class FCatSoundWaveContainerData;
class UCatSoundWaveContainer;

#define UE_API METASOUNDEXPERIMENTALENGINERUNTIME_API

UENUM(BlueprintType)
enum class ECatSoundWaveContainerType : uint8
{
	Standard,
	Random
};

USTRUCT(BlueprintType)
struct FCatSoundWaveContainerEntry
{
	GENERATED_BODY()

	FCatSoundWaveContainerEntry() = default;

	explicit FCatSoundWaveContainerEntry(USoundWave* InSoundWave)
		: SoundWave(InSoundWave)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Entry", meta = (ExactClass = true, AllowedClasses = "/Script/Engine.SoundWave"))
	TObjectPtr<USoundWave> SoundWave;

	UPROPERTY(EditAnywhere, Category = "Entry")
	float Weight = 1.0f;
};

// Named UCatSoundWaveContainer (not USoundWaveContainer) to avoid a UClass short-name
// collision with MetasoundPolyphonyInternal::USoundWaveContainer.
UCLASS(MinimalAPI, Blueprintable, BlueprintType, meta = (DisplayName = "Audio Sound Wave Container"))
class UCatSoundWaveContainer : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif

	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	/**
	 * Publish the current container state onto the proxy chain. Callable from code
	 * constructing a container in-memory (e.g. the test harness) so audio-thread
	 * consumers see the contents without waiting for an editor property change.
	 */
	UE_API void RebuildProxy();

	UPROPERTY(EditAnywhere, Category = Defaults)
	ECatSoundWaveContainerType Type = ECatSoundWaveContainerType::Standard;

	UPROPERTY(EditAnywhere, Category = Sounds)
	TArray<FCatSoundWaveContainerEntry> Entries;

private:
	TSharedPtr<FCatSoundWaveContainerProxy> Proxy;
};

class FCatSoundWaveContainerData : public Audio::ISoundWaveContainer
{
public:
	FCatSoundWaveContainerData() = default;
	FCatSoundWaveContainerData(const FCatSoundWaveContainerData& Other) = default;
	FCatSoundWaveContainerData(FCatSoundWaveContainerData&& Other) = default;

	UE_API explicit FCatSoundWaveContainerData(const Audio::FProxyDataInitParams& InInitParams);
	UE_API FCatSoundWaveContainerData(const Audio::FProxyDataInitParams& InInitParams, const UCatSoundWaveContainer& SoundWaveContainer);

	UE_API void Update(const UCatSoundWaveContainer& SoundWaveContainer);

	struct FEntry
	{
		FEntry() = default;
		UE_API FEntry(const Audio::FProxyDataInitParams& InInitParams, const FCatSoundWaveContainerEntry& Entry);

		TSharedPtr<FSoundWaveProxy> SoundWave;
		float Weight = 1.0f;
	};

	// Audio::ISoundWaveContainer.
	UE_API virtual TArray<FSoundWaveProxyPtr> GetContainedWaveProxies() const override;

	ECatSoundWaveContainerType Type = ECatSoundWaveContainerType::Standard;
	TArray<FEntry> Entries;
	Audio::FProxyDataInitParams InitParams;
};

// QueryInterface is hand-rolled (not IMPL_AUDIOPROXY_CLASS) so we can also answer
// Audio::ISoundWaveContainer by forwarding to the Data field.
class FCatSoundWaveContainerProxy : public Audio::TCatProxyView<FCatSoundWaveContainerProxy, FCatSoundWaveContainerData>
{
public:

	// Inherit constructor from TCatProxyView.
	using Audio::TCatProxyView<FCatSoundWaveContainerProxy, FCatSoundWaveContainerData>::TCatProxyView;

	static FName GetAudioProxyTypeName()
	{
		static FName MyClassName = TEXT("FCatSoundWaveContainerProxy");
		return MyClassName;
	}
	static FName GetInterfaceId() { return GetAudioProxyTypeName(); }
	static constexpr bool bWasAudioProxyClassImplemented = true;

	virtual void* QueryInterface(const FName InInterfaceId) override
	{
		if (InInterfaceId == GetAudioProxyTypeName())
		{
			return this;
		}
		if (InInterfaceId == Audio::ISoundWaveContainer::GetInterfaceId())
		{
			return static_cast<Audio::ISoundWaveContainer*>(&Data);
		}
		return IProxyData::QueryInterface(InInterfaceId);
	}

	friend class Audio::IProxyData;
	friend class Audio::TProxyData<FCatSoundWaveContainerProxy>;
};

#undef UE_API
