// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioProxyInitializer.h"
#include "MetasoundDataReference.h"
#include "MetasoundDataTypeRegistrationMacro.h"
#include "Templates/SharedPointer.h"

#define UE_API METASOUNDEXPERIMENTALENGINERUNTIME_API

class FCatSoundWaveContainerProxy;

namespace MetasoundCatExperimental
{
	/**
	 * MetaSound data-type wrapper around FCatSoundWaveContainerProxy. Acts as a read-side
	 * handle that the node operator holds by value on the audio thread; GetLatest()
	 * walks the proxy's linked list and caches the head so subsequent calls are cheap.
	 */
	class FSoundWaveContainerAsset
	{
	public:
		FSoundWaveContainerAsset() = default;
		FSoundWaveContainerAsset(const FSoundWaveContainerAsset&) = default;
		FSoundWaveContainerAsset& operator=(const FSoundWaveContainerAsset&) = default;

		UE_API explicit FSoundWaveContainerAsset(const TSharedPtr<Audio::IProxyData>& InData);

		/** Walk to the latest-published proxy and cache it; safe to call every block. */
		UE_API TSharedPtr<const FCatSoundWaveContainerProxy> GetLatest() const;

		void SetProxy(TSharedPtr<const FCatSoundWaveContainerProxy> InProxy)
		{
			Proxy = InProxy;
		}

		TSharedPtr<const FCatSoundWaveContainerProxy> GetProxy() const { return Proxy; }

		bool IsValid() const { return Proxy.IsValid(); }

		const FCatSoundWaveContainerProxy* operator->() const { return Proxy.Get(); }

	private:
		mutable TSharedPtr<const FCatSoundWaveContainerProxy> Proxy;
	};

	DECLARE_METASOUND_DATA_REFERENCE_ALIAS_TYPES(FSoundWaveContainerAsset, FSoundWaveContainerAssetTypeInfo, FSoundWaveContainerAssetReadRef, FSoundWaveContainerAssetWriteRef)
}

DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(MetasoundCatExperimental::FSoundWaveContainerAsset, METASOUNDEXPERIMENTALENGINERUNTIME_API);

#undef UE_API
