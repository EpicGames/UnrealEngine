// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "MetasoundFrontendDocument.h"
#include "MetasoundOperatorData.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "MetasoundCatWavePlayerNode.generated.h"

/**
 * Output-format selection mode for the Audio Wave Player.
 *
 * W10: scoped to this node (not the engine-wide enum) by intent for the duration of the
 * experimental phase. The cross-node enum EMetasoundChannelAgnosticNodeFormatChooser is
 * missing a "SourceAuto" variant (main spec §4 Open #7); localizing here avoids touching
 * MetasoundFrontend from an experimental-plugin module and keeps this node's config
 * self-contained. When the engine-wide enum grows a matching variant, the Wave Player
 * config should adopt it directly.
 */
UENUM()
enum class ECatWavePlayerFormatChooser : uint8
{
	/** Derive the output format from the contents of the attached SoundWaveContainer (largest channel count among entries, clamped to MVP formats). */
	Auto,

	/** Inherit the format from the enclosing UMetaSoundSource. Default. */
	SourceAuto,

	/** Pin a specific format via CustomFormat. Must be one of the MVP formats (Mono / Stereo / Quad / 5.1 / 7.1). */
	Custom
};

/** How the container is stepped through. */
UENUM()
enum class ECatWavePlayerPlaybackType : uint8
{
	/** Each Play trigger plays the entry at the supplied Index input. */
	Index,
	/** Each Play trigger advances through the container in sequence. */
	Sequence
};

/** How playback chooses entries (index-driven vs weighted-random). */
UENUM()
enum class ECatWavePlayerPlaybackMode : uint8
{
	/** Deterministic ordering: Index or Sequence input controls selection. */
	Standard,
	/** Weighted random: entry weights drive selection. */
	Random
};

/**
 * Configuration struct for the CAT Wave Player. See cat-wave-player-move-spec.md §3.2.
 * Drives both OverrideDefaultInterface (so the node UI reshapes to match the mode) and
 * GetOperatorData (so the operator knows its static configuration without round-tripping
 * through the vertex interface).
 */
USTRUCT()
struct FMetasoundCatWavePlayerNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	METASOUNDEXPERIMENTALENGINERUNTIME_API FMetasoundCatWavePlayerNodeConfiguration();

	/** Maximum number of simultaneous voices. Minimum enforced to 1. */
	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", UIMin = "1"))
	int32 MaxVoices = 2;

	/** Output-format derivation mode. See ECatWavePlayerFormatChooser. */
	UPROPERTY(EditAnywhere, Category = General)
	ECatWavePlayerFormatChooser Format = ECatWavePlayerFormatChooser::SourceAuto;

	/** Used when Format == Custom. Must be one of the MVP-registered formats (main spec §4 Confirmed #14). */
	UPROPERTY(EditAnywhere, Category = General, meta = (EditCondition = "Format == ECatWavePlayerFormatChooser::Custom", EditConditionHides))
	FName CustomFormat = TEXT("Cat:Stereo2Dot0");

	UPROPERTY(EditAnywhere, Category = General)
	ECatWavePlayerPlaybackType PlaybackType = ECatWavePlayerPlaybackType::Index;

	UPROPERTY(EditAnywhere, Category = General)
	ECatWavePlayerPlaybackMode PlaybackMode = ECatWavePlayerPlaybackMode::Standard;

	METASOUNDEXPERIMENTALENGINERUNTIME_API virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;
	METASOUNDEXPERIMENTALENGINERUNTIME_API virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};
