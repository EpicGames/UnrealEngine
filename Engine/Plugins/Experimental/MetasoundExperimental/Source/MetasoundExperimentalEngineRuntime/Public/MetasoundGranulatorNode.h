// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"
#include "TypeFamily/ChannelTypeFamily.h"
#include "UObject/PropertyText.h"

#include "MetasoundGranulatorNode.generated.h"

UCLASS()
class UMetaSoundGranulatorNodeOptionsHelper : public UObject
{
	GENERATED_BODY()

	UFUNCTION()
	static TArray<FPropertyTextFName> GetSoundFileFormatChannelOptions();
};


UENUM()
enum class EGranularEnvelope : uint8
{
	Gaussian = 0,
	Triangle,
	DownwardTriangle,
	UpwardTriangle,
	ExponentialDecay,
	ExponentialAttack,
	Hann,
	Count UMETA(Hidden)
};

namespace Metasound::GranulatorPrivate
{
	class FGranulatorOperatorData final : public TOperatorData<FGranulatorOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;

		explicit FGranulatorOperatorData(const FName& InOutputAudioTypeName, const EGranularEnvelope& InGranularEnvelope)
			: OutputAudioTypeName(InOutputAudioTypeName)
			, GranularEnvelope(InGranularEnvelope)
		{
		}

		FName OutputAudioTypeName;
		EGranularEnvelope GranularEnvelope;
	};
}

USTRUCT()
struct FMetaSoundGranulatorNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundGranulatorNodeConfiguration();

	// What the CAT output type is. Only azimuthal discrete channel types supported.
	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetaSoundGranulatorNodeOptionsHelper.GetSoundFileFormatChannelOptions"))
	FName OutputAudioTypeName = TEXT("Cat:Stereo2Dot0");

	// What grain envelope to use for the granulator
	UPROPERTY(EditAnywhere, Category = General)
	EGranularEnvelope GranularEnvelope = EGranularEnvelope::Hann;

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;

private:
	mutable TSharedPtr<Metasound::GranulatorPrivate::FGranulatorOperatorData> OperatorData;
};
