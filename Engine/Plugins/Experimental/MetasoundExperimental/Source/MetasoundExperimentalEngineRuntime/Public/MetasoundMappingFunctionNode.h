// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundOperatorData.h"
#include "MetasoundFrontendDocument.h"
#include "Curves/CurveFloat.h"

#include "MetasoundMappingFunctionNode.generated.h"


namespace Metasound::Experimental
{
	class FMappingFunctionNodeOperatorData : public TOperatorData<FMappingFunctionNodeOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;

		FMappingFunctionNodeOperatorData(const FRuntimeFloatCurve& InMappingFunction, bool bInWrapInputs)
			: MappingFunction(InMappingFunction)
			, bWrapInputs(bInWrapInputs)
		{
		}

		FRuntimeFloatCurve MappingFunction;
		bool bWrapInputs = false;
	};
}

USTRUCT()
struct FMetaSoundMappingFunctionNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundMappingFunctionNodeConfiguration();

	// Whether or not to wrap the input values to remain in the defined input range. Otherwise, inputs are clamped.
	UPROPERTY(EditAnywhere, Category = General)
	bool bWrapInputs = false;

	// Mapping function (curve) to map inputs to outputs
	UPROPERTY(EditAnywhere, Category = General)
	FRuntimeFloatCurve MappingFunction;

	TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;

private: 
	TSharedPtr<Metasound::Experimental::FMappingFunctionNodeOperatorData> OperatorData;
};
