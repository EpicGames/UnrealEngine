// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MetasoundExperimentalRuntime : ModuleRules
{
	public MetasoundExperimentalRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// PLEASE DO NOT ADD AN ENGINE DEPENDENCY HERE!
		// HORDE DOES NOT CATCH IT EITHER
		// RUN: .build program AudioUnitTests
		// RUN (ushell): .build program AudioUnitTests OR build in VS
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MetasoundGraphCore",
				"MetasoundFrontend",
				"AudioExperimentalRuntime"
			}
		);
		
		// When the Editor builds all plugins, it will build with engine. In that case add a dependency on CoreUObject.
		// If we don't do this we will get linker errors for the Engine expecting to find stuff defined here.
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("CoreUObject");
		}
		
		// PLEASE DO NOT ADD AN ENGINE DEPENDENCY HERE!
		// HORDE DOES NOT CATCH IT EITHER
		// RUN (ushell): .build program AudioUnitTests OR build in VS
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MetasoundGraphCore",
				"MetasoundFrontend",
				"MetasoundStandardNodes",
				"SignalProcessing",
				"Serialization",
				"Projects",
				"AudioChannelAgnosticCore"
			}
		);
		PrivateDefinitions.AddRange(
			new string[]
			{
				"METASOUND_PLUGIN=MetasoundExperimental",
				"METASOUND_MODULE=MetasoundExperimentalRuntime"
			});
		
		// These should probably move back to the experimental plugin again.
		RuntimeDependencies.Add("$(PluginDir)/Content/Schemas/ChannelAgnostic/Composite/*.json", StagedFileType.UFS);
		RuntimeDependencies.Add("$(PluginDir)/Content/Schemas/ChannelAgnostic/Discrete/*.json", StagedFileType.UFS);
		RuntimeDependencies.Add("$(PluginDir)/Content/Schemas/ChannelAgnostic/Soundfield/*.json", StagedFileType.UFS);
	}
}
