// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatBreakNode.h"

#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"
#include "DSP/MultiMono.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatBreakNode"

namespace Metasound
{
	class FChannelAgnosticType;
	struct FBuildOperatorParams;
	class FNodeFacade;

	namespace CatBreakNodePrivate
	{
		METASOUND_PARAM(InputFromCat,			"Input", "Cat to Break");

		static FName GetOutputPinName(const int32 InIndex, const Audio::FChannelTypeFamily& InFormat)
		{
			if (TOptional<Audio::FChannelTypeFamily::FChannelName> Names = InFormat.GetChannelName(InIndex); Names.IsSet())
			{
				return Names->Name;
			}
			const FName Output(TEXT("Output"), InIndex);
			return Output;
		}


		FOutputDataVertex MakeOutputDataVertex(const FName InName, const FString& InFriendlyName)
		{
			const FText DisplayName = METASOUND_LOCTEXT_FORMAT(
				"DisplayName_{0}", "{0}", FText::FromName(InName));

			const FText Tooltip = METASOUND_LOCTEXT_FORMAT(
				"Tooltip_{0}", "Channel {0}", FText::FromString(InFriendlyName));
			
			return FOutputDataVertex
			{
				InName,
				GetMetasoundDataTypeName<FAudioBuffer>(),
				FDataVertexMetadata
				{
					.Description = Tooltip,
					.DisplayName = DisplayName,
					.bIsAdvancedDisplay = false
				},
				EVertexAccessType::Reference
			};
		}
		
		
		FVertexInterface MakeClassInterface(const FName Format)
		{
			const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindConcreteChannel(Format);
			if (!Found)
			{
				return {};
			}
			const int32 NumChannels = Found->NumChannels();

			FInputVertexInterface InputInterface;
            InputInterface.Add(FInputDataVertex(METASOUND_GET_PARAM_NAME(InputFromCat), Format, METASOUND_GET_PARAM_METADATA(InputFromCat), EVertexAccessType::Reference));
			
			FOutputVertexInterface OutputInterface;
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex )
			{
				if (TOptional<Audio::FChannelTypeFamily::FChannelName> Name = Found->GetChannelName(ChannelIndex); Name.IsSet())
				{
					OutputInterface.Add(MakeOutputDataVertex(Name->Name, Name->FriendlyName));
				}
				else
				{
					const FName ChannelName(TEXT("Output"), ChannelIndex);
					OutputInterface.Add(MakeOutputDataVertex(ChannelName, FString::Printf(TEXT("Channel %d"), ChannelIndex)));	
				}
			}
			return{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
		
		class FCatBreakOperatorData final : public TOperatorData<FCatBreakOperatorData>
		{
		public:
			// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
			// that the downcast is valid.
			static const FLazyName OperatorDataTypeName;

			explicit FCatBreakOperatorData(const FName& InToTypeName, const Audio::EChannelTranscodeMethod InMethod, const Audio::EChannelMapMonoUpmixMethod InMixMethod)
				: ToTypeName(InToTypeName)
				, TranscodeMethod(InMethod)
			{}
			const FName& GetToType() const
			{
				return ToTypeName;
			}
			Audio::EChannelTranscodeMethod GetTranscodeMethod() const
			{
				return TranscodeMethod;
			}
			Audio::EChannelMapMonoUpmixMethod GetMixMethod() const
			{
				return MixMethod;
			}
		private:
			FName ToTypeName;
			Audio::EChannelTranscodeMethod TranscodeMethod = Audio::EChannelTranscodeMethod::ChannelDrop;
			Audio::EChannelMapMonoUpmixMethod MixMethod = Audio::EChannelMapMonoUpmixMethod::EqualPower; 
		};

		// Linkage.
		const FLazyName FCatBreakOperatorData::OperatorDataTypeName = TEXT("FCatBreakOperatorData");

		// Helper to create array of multi-mono channel pointers from a CAT.
		Audio::TStackArrayOfPointers<float> MakeMultiMonoPointersFromBufferArray(const TArray<TDataWriteReference<FAudioBuffer>>& InArrayOfCat)
		{
			Audio::TStackArrayOfPointers<float> Result;
			Result.SetNum(InArrayOfCat.Num());
			for (int32 i = 0; i < InArrayOfCat.Num(); ++i)
			{
				Result[i] = InArrayOfCat[i]->GetData();
			}
			return Result;
		}
	}
	
	class FCatBreakOperator final : public TExecutableOperator<FCatBreakOperator>
	{
	public:
		using FTranscoder = Audio::ChannelAgnosticTranscoder::FTranscoder; 
		FCatBreakOperator(const FBuildOperatorParams& InParams, FChannelAgnosticTypeReadRef&& InInputCat, TArray<TDataWriteReference<FAudioBuffer>>&& InOutputVerticies,
			FTranscoder&& InTranscoder, const FName InFormat)
			: InputCat(MoveTemp(InInputCat))
			, OutputAudioVertices(MoveTemp(InOutputVerticies))
			, Settings(InParams.OperatorSettings)
			, Transcoder(InTranscoder)
			, Format(InFormat)
		{}
		virtual ~FCatBreakOperator() override = default;
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CatBreakNodePrivate;

			const FCatBreakOperatorData* CatBreakData = CastOperatorData<const FCatBreakOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!CatBreakData)
			{
				return MakeUnique<FNoOpOperator>();
			}
			
			const FName OutputFormat = CatBreakData->GetToType();
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const Audio::FChannelTypeFamily* ConcreteToType = Audio::GetChannelRegistry().FindConcreteChannel(OutputFormat);
			if (!ConcreteToType)
			{
				// Throw editor error, we don't know the format.
				return MakeUnique<FNoOpOperator>();
			}

			// Create the input pin.
			TDataReadReference<FChannelAgnosticType> InputPin = InputData.GetOrCreateDefaultDataReadReference<FChannelAgnosticType>(
				METASOUND_GET_PARAM_NAME(InputFromCat), InParams.OperatorSettings);

			// Create output based on format.
			TArray<TDataWriteReference<FAudioBuffer>> OutputAudioVertices;
			for (int32 i = 0; i < ConcreteToType->NumChannels(); ++i)
			{
				OutputAudioVertices.Emplace(FAudioBufferWriteRef::CreateNew(InParams.OperatorSettings));
			}

			// Always ask for the transcoder 
			// In the trivial case where we are the same format, this will be just a memcpy.
			FTranscoder Transcoder = Audio::FTranscoderResolver::Resolve(
				InputPin->GetType(),
				{
					.ToType = *ConcreteToType,
					.TranscodeMethod = CatBreakData->GetTranscodeMethod(),
					.MixMethod = CatBreakData->GetMixMethod(),
				});
			
			return MakeUnique<FCatBreakOperator>(
				InParams,
				MoveTemp(InputPin),
				MoveTemp(OutputAudioVertices),
				MoveTemp(Transcoder),
				ConcreteToType->GetName()
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatBreakNodePrivate;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFromCat),InputCat);
		}
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatBreakNodePrivate;
			for (int32 i = 0; i < OutputAudioVertices.Num(); i++)
			{
				InOutVertexData.BindWriteVertex(GetOutputPinName(i, InputCat->GetType()), OutputAudioVertices[i]);
			}
		}

		void Reset(const FResetParams& InParams)
		{
			Execute();
		}

		void Execute()
		{
			using namespace Audio;
			using namespace CatBreakNodePrivate;
			if (Transcoder)
			{
				TStackArrayOfPointers<const float> Src = MakeMultiMonoPointersFromView(InputCat->GetRawMultiMono(), Settings.GetNumFramesPerBlock(), InputCat->NumChannels());
				TStackArrayOfPointers<float> Dst = MakeMultiMonoPointersFromBufferArray(OutputAudioVertices);
				Transcoder(Src, Dst, Settings.GetNumFramesPerBlock() );
			}
		}
		
		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace CatBreakNodePrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "CatBreakOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("CatBreakNodeName", "CAT Break Node"),	
				METASOUND_LOCTEXT("CatBreakNodeNameDescription", "A Node that Breaks CATs"),	
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("ExampleConfigurablePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				MakeClassInterface(TEXT("Cat:Stereo2Dot0")),
				{}
			};
		}
	private:
		FChannelAgnosticTypeReadRef InputCat;
		TArray<TDataWriteReference<FAudioBuffer>> OutputAudioVertices;
		FOperatorSettings Settings;
		FTranscoder Transcoder;
		FName Format;
	}; // class FCatCastingOperator
		
	using FCatBreakNode = TNodeFacade<FCatBreakOperator>;

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatBreakNode, FMetaSoundCatBreakNodeConfiguration);
} // namespace Metasound


TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatBreakNodeConfiguration::OverrideDefaultInterface(
	const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound::CatBreakNodePrivate;
	return  TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(MakeClassInterface(Format)));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatBreakNodeConfiguration::GetOperatorData() const
{
	return MakeShared<Metasound::CatBreakNodePrivate::FCatBreakOperatorData>(
		Format,
		static_cast<Audio::EChannelTranscodeMethod>(TranscodeMethod),
		static_cast<Audio::EChannelMapMonoUpmixMethod>(MixMethod)
	);
}

#undef LOCTEXT_NAMESPACE // 
