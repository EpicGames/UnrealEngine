// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"
#include "MetasoundCatCastingNode.h"

#include "MetasoundCatBreakNode.generated.h"

USTRUCT()
struct FMetaSoundCatBreakNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatBreakNodeConfiguration() = default;

	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions_NoAbstract"))
	FName Format = TEXT("Cat:Stereo2Dot0");

	UPROPERTY(EditAnywhere, Category = General)
	EMetasoundCatCastingMethod TranscodeMethod = EMetasoundCatCastingMethod::ChannelDrop;

	UPROPERTY(EditAnywhere, Category = General, meta = (EditCondition = "TranscodeMethod == EMetasoundCatCastingMethod::MixUpOrDown", EditConditionHides))
	EMetasoundChannelMapMonoUpmixMethod MixMethod = EMetasoundChannelMapMonoUpmixMethod::EqualPower; 

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

