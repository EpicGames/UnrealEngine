// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatCastingNode.h"

#include "MetasoundChannelAgnosticDataSchemas.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"
#include "DSP/MultiMono.h"
#include "TypeFamily/ChannelTypeFamily.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatCastingNode"

namespace Metasound
{
	class FChannelAgnosticType;
	struct FBuildOperatorParams;
	class FNodeFacade;

	namespace CatCastingPrivate
	{
		METASOUND_PARAM(InputFromCat,			"Input", "CAT to Cast");
		METASOUND_PARAM(OutputToCat,			"Output", "CAT Result");
		
		class FCatCastingOperatorData final : public TOperatorData<FCatCastingOperatorData>
		{
		public:
			// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
			// that the downcast is valid.
			static const FLazyName OperatorDataTypeName;

			explicit FCatCastingOperatorData(const FName& InToTypeName, Audio::EChannelTranscodeMethod InTranscodeMethod, Audio::EChannelMapMonoUpmixMethod InMixMethod)
				: ToTypeName(InToTypeName)
				, TranscodeMethod(InTranscodeMethod)
				, MixMethod(InMixMethod)
			{}
			const FName& GetToType() const
			{
				return ToTypeName;
			}
			Audio::EChannelMapMonoUpmixMethod GetMixMethod() const { return MixMethod; }
			Audio::EChannelTranscodeMethod GetTranscodeMethod() const { return TranscodeMethod; }
		private:
			FName ToTypeName;
			Audio::EChannelTranscodeMethod TranscodeMethod;
			Audio::EChannelMapMonoUpmixMethod MixMethod;
		};

		// Linkage.
		const FLazyName FCatCastingOperatorData::OperatorDataTypeName = TEXT("FCatCastingOperatorData");
	}

	static TDataWriteReference<FChannelAgnosticType>
	MakePolyCat(const FName InDerivedType, const FOperatorSettings& InOpSettings)
	{
		using namespace Frontend;
		const FLiteral Literal(InDerivedType.ToString());
		TOptional<FAnyDataReference> Ref = IDataTypeRegistry::Get().CreateDataReference(InDerivedType, EDataReferenceAccessType::Write, Literal, InOpSettings);
		return Ref->GetAs<TDataWriteReference<FChannelAgnosticType>>();
	}

	FCatCastingOperator::FCatCastingOperator(const FBuildOperatorParams& InParams, FChannelAgnosticTypeReadRef&& InInputCat,
	                                         const CatCastingPrivate::FCatCastingOperatorData& InData, const FName InConcreteName)
		: InputFrom(MoveTemp(InInputCat))
		, ToFormatName(InConcreteName)
		, OutputCastResult(MakePolyCat(InConcreteName, InParams.OperatorSettings))
		, Settings(InParams.OperatorSettings)
		, TranscodeMethod(InData.GetTranscodeMethod())
		, MixMethod(InData.GetMixMethod())
	{
	}

	FVertexInterface FCatCastingOperator::GetInterface(const FName InSrc, const FName InDst) 
	{
		using namespace CatCastingPrivate;

		// inputs
		FInputVertexInterface InputInterface;
		InputInterface.Add(
			FInputDataVertex(
				METASOUND_GET_PARAM_NAME(InputFromCat),
				InSrc,
				METASOUND_GET_PARAM_METADATA(InputFromCat)
			)
		);
		
		// outputs
		FOutputVertexInterface OutputInterface;
		OutputInterface.Add(
			FOutputDataVertex(
				METASOUND_GET_PARAM_NAME(OutputToCat),
				InDst,
				METASOUND_GET_PARAM_METADATA(OutputToCat),
				EVertexAccessType::Reference
			)
		);
		
		return FVertexInterface(InputInterface, OutputInterface);
	}

	TUniquePtr<IOperator> FCatCastingOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
	{
		using namespace CatCastingPrivate;

		const FCatCastingOperatorData* CatTestingConfigData = CastOperatorData<const FCatCastingOperatorData>(InParams.Node.GetOperatorData().Get());
		if (!CatTestingConfigData)
		{
			return MakeUnique<FNoOpOperator>();
		}
		const FName CastToName = CatTestingConfigData->GetToType();

		const FInputVertexInterfaceData& InputData = InParams.InputData;
		const Audio::FChannelTypeFamily* ConcreteToType = Audio::GetChannelRegistry().FindConcreteChannel(CastToName);

		TDataReadReference<FChannelAgnosticType> InputCat = InputData.GetOrCreateDefaultDataReadReference<FChannelAgnosticType>(
			METASOUND_GET_PARAM_NAME(InputFromCat), InParams.OperatorSettings);

		// Make sure the cast is to something sane, otherwise use the inputs type... 
		const FName CastToNameSane = ConcreteToType ? ConcreteToType->GetName() : InputCat->GetTypeName();

		return MakeUnique<FCatCastingOperator>(
			InParams,
			MoveTemp(InputCat),
			*CatTestingConfigData,
			CastToNameSane
		);
	}

	void FCatCastingOperator::BindInputs(FInputVertexInterfaceData& InOutVertexData) 
	{
		using namespace CatCastingPrivate;
		InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputFromCat), InputFrom);
	}

	void FCatCastingOperator::BindOutputs(FOutputVertexInterfaceData& InOutVertexData) 
	{
		using namespace CatCastingPrivate;
		InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputToCat), OutputCastResult);

		// Create transcoder 
		Transcoder = Audio::FTranscoderResolver::Resolve(
			InputFrom->GetType(),
			{
				.ToType = OutputCastResult->GetType(),
				.TranscodeMethod = TranscodeMethod,
				.MixMethod = MixMethod,
			}
		);
	}

	void FCatCastingOperator::Reset(const FResetParams& InParams)
	{
		Execute();
	}

	void FCatCastingOperator::Execute()
	{
		if (Transcoder)
		{
			using namespace CatCastingPrivate;
			using namespace Audio;
			TStackArrayOfPointers<const float> Src = MakeMultiMonoPointersFromView(InputFrom->GetRawMultiMono(), Settings.GetNumFramesPerBlock(),
			                                                                       InputFrom->NumChannels());
			TStackArrayOfPointers<float> Dst = MakeMultiMonoPointersFromView(OutputCastResult->GetRawMultiMono(), Settings.GetNumFramesPerBlock(),
			                                                                 OutputCastResult->NumChannels());
			Transcoder(Src, Dst, Settings.GetNumFramesPerBlock());
		}
	}

	FNodeClassMetadata FCatCastingOperator::GetNodeInfo()
	{
		using namespace CatCastingPrivate;
		static FName DefaultFormat = TEXT("Cat");
		return FNodeClassMetadata
		{
			FNodeClassName{"Experimental", "CatCastingOperator", ""},
			1, // Major version
			0, // Minor version
				METASOUND_LOCTEXT("CatCastingNodeName", "CAT Casting Node"),	
				METASOUND_LOCTEXT("CatCastingNodeNameDescription", "A Node that allows Casting to CATs"),	
			TEXT("UE"), // Author
				METASOUND_LOCTEXT("ExampleConfigurablePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
			GetInterface(DefaultFormat, DefaultFormat),
			{}
		};
	}

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatCastingNode, FMetaSoundCatCastingNodeConfiguration);
}; // class FCatCastingOperator




TArray<FPropertyTextFName> UMetasoundCatCastingOptionsHelper::GetCastingOptions()
{
	TArray<TSharedRef<const Audio::FChannelTypeFamily>> AllFormats = Audio::GetChannelRegistry().GetAllChannelFormats();
	TArray<FPropertyTextFName> FormatsOptions;
	Algo::Transform(AllFormats, FormatsOptions, [](const TSharedRef<const Audio::FChannelTypeFamily>& i) -> FPropertyTextFName
		{ return {.ValueString = i->GetName(), .DisplayName = FText::FromString(i->GetFriendlyName()) }; });
	return FormatsOptions;
}

TArray<FPropertyTextFName> UMetasoundCatCastingOptionsHelper::GetCastingOptions_NoAbstract()
{
	TArray<TSharedRef<const Audio::FChannelTypeFamily>> AllFormats = Audio::GetChannelRegistry().GetAllChannelFormats();
	TArray<FPropertyTextFName> FormatsOptions;
	Algo::TransformIf(
		AllFormats,
		FormatsOptions,
		[](const TSharedRef<const Audio::FChannelTypeFamily>& i) -> bool { return !i->IsAbstract(); },
		[](const TSharedRef<const Audio::FChannelTypeFamily>& i) -> FPropertyTextFName
		{ return {.ValueString = i->GetName(), .DisplayName = FText::FromString(i->GetFriendlyName()) }; });
	return FormatsOptions;
}

TArray<FPropertyTextFName> UMetasoundCatCastingOptionsHelper::GetCastingOptions_DiscreteOnly()
{
	// All non-abstract discrete formats. Excludes soundfield (Atmos, ambisonics) and composite types.
	struct FDiscreteChecker : public Audio::IChannelTypeVisitor
	{
		bool bIsDiscrete = false;
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		void Visit(const Audio::FDiscreteChannelTypeFamily&) override { bIsDiscrete = true; }
		void Visit(const Audio::FSoundfieldChannelTypeFamily&) override { bIsDiscrete = false; }
		void Visit(const Audio::FCompositeChannelTypeFamily&) override { bIsDiscrete = false; }
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	};

	TArray<TSharedRef<const Audio::FChannelTypeFamily>> AllFormats = Audio::GetChannelRegistry().GetAllChannelFormats();
	TArray<FPropertyTextFName> FormatsOptions;
	for (const TSharedRef<const Audio::FChannelTypeFamily>& Format : AllFormats)
	{
		if (Format->IsAbstract())
		{
			continue;
		}
		FDiscreteChecker Checker;
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		Format->Accept(Checker);
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		if (Checker.bIsDiscrete)
		{
			FormatsOptions.Add({.ValueString = Format->GetName(), .DisplayName = FText::FromString(Format->GetFriendlyName())});
		}
	}
	return FormatsOptions;
}

TArray<FPropertyTextFName> UMetasoundCatCastingOptionsHelper::GetCastingOptions_AudioMixerOnly()
{
	// Only the standard AudioMixer formats: Mono 1.0, Stereo 2.0, Quad 4.0, 5.1, 7.1.
	// These are the exact formats AudioMixer supports for buffer allocation and interleaving.
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	static constexpr int32 MixerChannelCounts[] = { 1, 2, 4, 6, 8 };
	TArray<FPropertyTextFName> FormatsOptions;
	for (int32 NumCh : MixerChannelCounts)
	{
		if (const TSharedPtr<const Audio::FDiscreteChannelTypeFamily> Format =
			Audio::FChannelAgnosticUtils::FindDiscreteFormatFromNumChannels(NumCh))
		{
			FormatsOptions.Add({.ValueString = Format->GetName(), .DisplayName = FText::FromString(Format->GetFriendlyName())});
		}
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	return FormatsOptions;
}

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatCastingNodeConfiguration::OverrideDefaultInterface(
	const FMetasoundFrontendClass& InNodeClass) const
{
	static const FName DefaultInput = TEXT("Cat");
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(
		FMetasoundFrontendClassInterface::GenerateClassInterface(
			Metasound::FCatCastingOperator::GetInterface(DefaultInput,  Metasound::AddCatNamespaceToName(ToType))));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatCastingNodeConfiguration::GetOperatorData() const
{
	return MakeShared<Metasound::CatCastingPrivate::FCatCastingOperatorData>(
		ToType,
		static_cast<Audio::EChannelTranscodeMethod>(TranscodeMethod),
		static_cast<Audio::EChannelMapMonoUpmixMethod>(MixMethod)
		);
}

#undef LOCTEXT_NAMESPACE // 
