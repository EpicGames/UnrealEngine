// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFrontendDocument.h"
#include "ChannelAgnostic/ChannelAgnosticTranscoding.h"
#include "Templates/SharedPointer.h"
#include "TypeFamily/ChannelTypeFamily.h"
#include "UObject/PropertyText.h"

#include "MetasoundCatCastingNode.generated.h"

UCLASS()
class UMetasoundCatCastingOptionsHelper : public UObject
{
	GENERATED_BODY()

	UFUNCTION()
	static TArray<FPropertyTextFName> GetCastingOptions();

	UFUNCTION()
	static TArray<FPropertyTextFName> GetCastingOptions_NoAbstract();

	// Only discrete (mixer-compatible) formats. Excludes soundfield, ambisonics, composite.
	UFUNCTION()
	static TArray<FPropertyTextFName> GetCastingOptions_DiscreteOnly();

	// Only the standard AudioMixer formats: Mono 1.0, Stereo 2.0, Quad 4.0, 5.1, 7.1.
	// For MetaSound Source custom format dropdown — must match what AudioMixer can interleave.
	UFUNCTION()
	static TArray<FPropertyTextFName> GetCastingOptions_AudioMixerOnly();
};

UENUM()
enum class EMetasoundCatCastingMethod : uint8
{
	ChannelDrop = static_cast<uint8>(Audio::EChannelTranscodeMethod::ChannelDrop),
	MixUpOrDown = static_cast<uint8>(Audio::EChannelTranscodeMethod::MixUpOrDown)
};

UENUM()
enum class EMetasoundChannelMapMonoUpmixMethod : uint8
{
	Linear = static_cast<uint8>(Audio::EChannelMapMonoUpmixMethod::Linear),
	EqualPower = static_cast<uint8>(Audio::EChannelMapMonoUpmixMethod::EqualPower),
	FullVolume = static_cast<uint8>(Audio::EChannelMapMonoUpmixMethod::FullVolume)
};

USTRUCT()
struct FMetaSoundCatCastingNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatCastingNodeConfiguration() =  default;

	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions_NoAbstract"))
	FName ToType = TEXT("Mono");
	
	UPROPERTY(EditAnywhere, Category = General)
	EMetasoundCatCastingMethod TranscodeMethod = EMetasoundCatCastingMethod::ChannelDrop;

	UPROPERTY(EditAnywhere, Category = General, meta = (EditCondition = "TranscodeMethod == EMetasoundCatCastingMethod::MixUpOrDown", EditCOonditionHides))
	EMetasoundChannelMapMonoUpmixMethod MixMethod = EMetasoundChannelMapMonoUpmixMethod::EqualPower; 

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

namespace Metasound
{
	namespace CatCastingPrivate
	{
		class FCatCastingOperatorData;
	}

	struct FBuildOperatorParams;

	class FCatCastingOperator final : public TExecutableOperator<FCatCastingOperator>
	{
	public:
		FCatCastingOperator(const FBuildOperatorParams& InParams, FChannelAgnosticTypeReadRef&& InInputCat, const CatCastingPrivate::FCatCastingOperatorData& InData, const FName InConcreteName);
		virtual ~FCatCastingOperator() override = default;

		static FVertexInterface GetInterface(const FName InSrc, const FName InDst);
			
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override;
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override;

		void Reset(const FResetParams& InParams);

		void Execute();
	
		static FNodeClassMetadata GetNodeInfo();
	
	private:
		using FTranscoder = Audio::ChannelAgnosticTranscoder::FTranscoder;
		FChannelAgnosticTypeReadRef InputFrom;
		FName ToFormatName;
		TDataWriteReference<FChannelAgnosticType> OutputCastResult;
		FOperatorSettings Settings;
		FTranscoder Transcoder;
		Audio::EChannelTranscodeMethod TranscodeMethod;
		Audio::EChannelMapMonoUpmixMethod MixMethod;
	}; // class FCatCastingOperator

	using FCatCastingNode = TNodeFacade<FCatCastingOperator>;

}		
