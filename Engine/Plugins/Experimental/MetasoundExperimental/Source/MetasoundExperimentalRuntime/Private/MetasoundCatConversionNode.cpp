// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatConversionNode.h"

#include "MetasoundCatCastingNode.h"
#include "MetasoundFrontendNodesCategories.h"
#include "MetasoundNodeInterface.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"

#define LOCTEXT_NAMESPACE "MetasoundFrontend"

namespace Metasound
{
	struct FConvertDataTypeInfo
	{
		FName FromDataTypeName;
		FText FromDataTypeText;
		FName ToDataTypeName;
		FText ToDataTypeText;
	};
	FVertexName GetInputName(const FConvertDataTypeInfo& InInfo)
	{
		return InInfo.FromDataTypeName;
	}

	FVertexName GetOutputName(const FConvertDataTypeInfo& InInfo)
	{
		return InInfo.ToDataTypeName;
	}

	FVertexInterface CreateVertexInterface(const FConvertDataTypeInfo& InInfo)
	{
		const FText InputDesc = {}; //METASOUND_LOCTEXT_FORMAT("AutoConvDisplayNamePatternFrom", "Input {0} value.", InInfo.FromDataTypeText);
		const FText OutputDesc = {};//METASOUND_LOCTEXT_FORMAT("AutoConvDisplayNamePatternTo", "Output {0} value.", InInfo.ToDataTypeText);

		return FVertexInterface(
			FInputVertexInterface(
				FInputDataVertex(GetInputName(InInfo), InInfo.FromDataTypeName, FDataVertexMetadata{ InputDesc }, EVertexAccessType::Reference)
			),
			FOutputVertexInterface(
				FOutputDataVertex(GetOutputName(InInfo), InInfo.ToDataTypeName, FDataVertexMetadata{ OutputDesc }, EVertexAccessType::Reference)
			)
		);
	}
	FNodeClassMetadata CreateAutoConverterNodeMetadata(const FConvertDataTypeInfo& InInfo)
	{
		FNodeDisplayStyle DisplayStyle;
		DisplayStyle.bShowName = false;
		DisplayStyle.ImageName = TEXT("MetasoundEditor.Graph.Node.Conversion");
		DisplayStyle.bShowInputNames = false;
		DisplayStyle.bShowOutputNames = false;

		const FText FromTypeText = InInfo.FromDataTypeText;
		const FText ToTypeText = InInfo.ToDataTypeText;

		FNodeClassMetadata Info;
		Info.ClassName = {TEXT("Convert"), InInfo.ToDataTypeName, InInfo.FromDataTypeName};
		Info.MajorVersion = 1;
		Info.MinorVersion = 0;
		/*
		Info.DisplayName = METASOUND_LOCTEXT_FORMAT("Metasound_AutoConverterNodeDisplayNameFormat", "{0} to {1}", InInfo.FromDataTypeText,
													InInfo.ToDataTypeText);
		Info.Description = METASOUND_LOCTEXT_FORMAT("Metasound_AutoConverterNodeDescriptionNameFormat", "Converts from {0} to {1}.",
													InInfo.FromDataTypeText, InInfo.ToDataTypeText);
													*/
		Info.Author = PluginAuthor;
		Info.DisplayStyle = DisplayStyle;
		Info.PromptIfMissing = PluginNodeMissingPrompt;
		Info.DefaultInterface = CreateVertexInterface(InInfo);

		Info.CategoryHierarchy.Emplace(NodeCategories::Conversions);

		Info.Keywords =
		{
			METASOUND_LOCTEXT("MetasoundConvertKeyword", "Convert"),
			InInfo.FromDataTypeText,
			InInfo.ToDataTypeText
		};

		return Info;
	}

	void RegisterCatConvertNode()
	{
		FConvertDataTypeInfo DataTypeInfo
		{
			.FromDataTypeName = TEXT("Cat"),
			.FromDataTypeText = FText::FromString(TEXT("Cat")),
			.ToDataTypeName = TEXT("Cat:Discrete"),
			.ToDataTypeText = FText::FromString(TEXT("Cat:Discrete")),
		};

		const FNodeClassMetadata& Metadata = Metasound::CreateAutoConverterNodeMetadata(DataTypeInfo);
		const Frontend::FNodeRegistryKey Key(Metadata);

		const Frontend::FConverterNodeClassInfo CatCast
		{
			// If this node has multiple input pins, we use this to designate which pin should be used.
			.PreferredConverterInputPin = TEXT("Cat"),

			// If this node has multiple output pins, we use this to designate which pin should be used.
			.PreferredConverterOutputPin = TEXT("Cat"),

			// The key for this node in the node registry.
			.NodeKey = Key
		};
		Frontend::FConverterNodeClassRegistryKey ConverterNodeKey
		{
			.FromDataType = TEXT("Cat"),
			.ToDataType = TEXT("Cat:Discrete"),
		};

		FMetasoundFrontendRegistryContainer::Get()->RegisterConversionNode(ConverterNodeKey, CatCast);
	}
}

#undef LOCTEXT_NAMESPACE 
