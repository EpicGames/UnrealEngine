// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatMakeNode.h"

#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatMakeNode"

namespace Metasound
{
	class FChannelAgnosticType;
	struct FBuildOperatorParams;
	class FNodeFacade;

	namespace CatMakeNodePrivate
	{
		METASOUND_PARAM(InputFromCat,			"Input", "CAT to Cast");
		METASOUND_PARAM(OutputToCat,			"Output", "CAT Result");

		FInputDataVertex MakeInputDataVertex(const FName InName, const FString& InFriendlyName)
		{
			const FText DisplayName = METASOUND_LOCTEXT_FORMAT(
				"DisplayName_{0}", "{0}", FText::FromName(InName));

			const FText Tooltip = METASOUND_LOCTEXT_FORMAT(
				"Tooltip_{0}", "Channel {0}", FText::FromString(InFriendlyName));
			
			return FInputDataVertex
			{
				InName,
				GetMetasoundDataTypeName<FAudioBuffer>(),
				FDataVertexMetadata
				{
					.Description = Tooltip,
					.DisplayName = DisplayName,
					.bIsAdvancedDisplay = false
				}
			};
		}
		
		static FVertexInterface MakeClassInterface(const FName Format)
		{
			const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindConcreteChannel(Format);
			if (!Found)
			{
				return {};
			}
			const int32 NumChannels = Found->NumChannels();

			FInputVertexInterface InputInterface;
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex )
			{
				if (TOptional<Audio::FChannelTypeFamily::FChannelName> Name = Found->GetChannelName(ChannelIndex); Name.IsSet())
				{
					InputInterface.Add(MakeInputDataVertex(Name->Name, Name->FriendlyName));
				}
				else
				{
					const FName Input("In", ChannelIndex);
					InputInterface.Add(MakeInputDataVertex(Input, FString::FromInt(ChannelIndex) ));	
				}
			}
			
			// outputs
			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(
				FOutputDataVertex(
					METASOUND_GET_PARAM_NAME(OutputToCat),
					Format,
					METASOUND_GET_PARAM_METADATA(OutputToCat),
					EVertexAccessType::Reference
				)
			);
			return{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
		
		class FCatMakeOperatorData final : public TOperatorData<FCatMakeOperatorData>
		{
		public:
			// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
			// that the downcast is valid.
			static const FLazyName OperatorDataTypeName;

			explicit FCatMakeOperatorData(const FName& InToTypeName)
				: ToTypeName(InToTypeName)
			{}
			const FName& GetToType() const
			{
				return ToTypeName;
			}
		private:
			FName ToTypeName;
		};

		// Linkage.
		const FLazyName FCatMakeOperatorData::OperatorDataTypeName = TEXT("FCatMakeOperatorData");
	}
	
	class FCatMakeOperator final : public TExecutableOperator<FCatMakeOperator>
	{
	public:
		FCatMakeOperator(const FBuildOperatorParams& InParams, TArray<TDataReadReference<FAudioBuffer>>&& InInputs, FChannelAgnosticTypeWriteRef&& InOutputCat)
			: InputAudioVertices(MoveTemp(InInputs))
			, OutputCat(MoveTemp(InOutputCat))
			, Settings(InParams.OperatorSettings)
		{
		}
		virtual ~FCatMakeOperator() override = default;

		static TDataWriteReference<FChannelAgnosticType>
		MakePolyCat(const FName InDerivedType, const FOperatorSettings& InOpSettings)
		{
			using namespace Frontend;
			const FLiteral Literal(InDerivedType.ToString());
			TOptional<FAnyDataReference> Ref = IDataTypeRegistry::Get().CreateDataReference(InDerivedType, EDataReferenceAccessType::Write, Literal, InOpSettings);
			return Ref->GetAs<TDataWriteReference<FChannelAgnosticType>>();
		}

		static FName GetPinName(const int32 InIndex, const Audio::FChannelTypeFamily& InFormat)
		{
			if (auto Names = InFormat.GetChannelName(InIndex))
			{
				return Names->Name;
			}
			FName Input;
			Input.SetNumber(InIndex);
			return Input;
		}
		
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CatMakeNodePrivate;

			const FCatMakeOperatorData* CatMakeData = CastOperatorData<const FCatMakeOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!CatMakeData)
			{
				return TUniquePtr<FNoOpOperator>();
			}
			const FName OutputFormat = CatMakeData->GetToType();
			const FInputVertexInterfaceData& InputData = InParams.InputData;
			const Audio::FChannelTypeFamily* ConcreteToType = Audio::GetChannelRegistry().FindConcreteChannel(OutputFormat);
			if (!ConcreteToType)
			{
				return TUniquePtr<FNoOpOperator>();
			}

			// Set the number of inputs pins to match the output format.
			TArray<TDataReadReference<FAudioBuffer>> InputAudioVertices;
			for (int32 i = 0; i < ConcreteToType->NumChannels(); ++i)
			{
				InputAudioVertices.Emplace(InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(GetPinName(i, *ConcreteToType), InParams.OperatorSettings));
			}
						
			// Make output CAT to match our configs settings. (use concrete form).
			FChannelAgnosticTypeWriteRef OutputPin = MakePolyCat(ConcreteToType->GetName(), InParams.OperatorSettings);
			
			return MakeUnique<FCatMakeOperator>(
				InParams,
				MoveTemp(InputAudioVertices),
				MoveTemp(OutputPin)
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatMakeNodePrivate;
			for (int32 i = 0; i < InputAudioVertices.Num(); i++)
			{
				InOutVertexData.BindReadVertex(GetPinName(i, OutputCat->GetType()), InputAudioVertices[i]);
			}
		}
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatMakeNodePrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputToCat),OutputCat);
		}

		void Reset(const FResetParams& InParams)
		{
			Execute();
		}

		void Execute()
		{
			// Copy each input channel into the CAT.
			for (int32 Index = 0; Index < InputAudioVertices.Num(); Index++)
			{
				const FAudioBuffer& Src = *InputAudioVertices[Index]; 
				TArrayView<float> Dst = OutputCat->GetChannel(Index);
				checkSlow(Src.Num() == Dst.Num());
				FMemory::Memcpy(Dst.GetData(), Src.GetData(), Dst.Num() * sizeof(float));

				// REWRITE ME TO BE REFERENCE FORWARD.
			}
		}
		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace CatMakeNodePrivate;
			static FName DefaultFormat = TEXT("Cat:Mono1Dot0");
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "CatMakeOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("CatMakeNodeName", "CAT Make Node"),	
				METASOUND_LOCTEXT("CatMakeNodeNameDescription", "A Node that builds CATs"),	
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("CatMakeNodePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				MakeClassInterface(DefaultFormat),
				{}
			};
		}
	private:
		TArray<TDataReadReference<FAudioBuffer>> InputAudioVertices;
		FChannelAgnosticTypeWriteRef OutputCat;
		FOperatorSettings Settings;
	}; // class FCatCastingOperator
		
	using FCatMakeNode = TNodeFacade<FCatMakeOperator>;

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatMakeNode, FMetaSoundCatMakeNodeConfiguration);
} // namespace Metasound


TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatMakeNodeConfiguration::OverrideDefaultInterface(
	const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound::CatMakeNodePrivate;
	return  TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(MakeClassInterface(Format)));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatMakeNodeConfiguration::GetOperatorData() const
{
	return MakeShared<Metasound::CatMakeNodePrivate::FCatMakeOperatorData>(Format);
}

#undef LOCTEXT_NAMESPACE // 
