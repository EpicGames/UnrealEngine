// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"

#include "MetasoundCatMakeNode.generated.h"

USTRUCT()
struct FMetaSoundCatMakeNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatMakeNodeConfiguration() = default;

	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions_NoAbstract"))
	FName Format = TEXT("Cat:Stereo");

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

