// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatMixerNode.h"

#include "Algo/MaxElement.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/MultiMono.h"
#include "MetasoundCatBreakNode.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundVertex.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatMixerNode"

namespace Metasound
{
	namespace CatMixerPrivate
	{
		METASOUND_PARAM(OutputCat,			"Cat Out", "Channel Agnostic Output");
		
		class FCatMixerOperatorData final : public TOperatorData<FCatMixerOperatorData>
		{
		public:
			static const FLazyName OperatorDataTypeName;
			FCatMixerOperatorData(const FName InToMixType, const int32 InNumInputs, const EMetasoundMixerFormatChoosingMethod InFormatChosingMethod, const EMetasoundChannelMapMonoUpmixMethod InUpMixMethod, const EMetasoundCatCastingMethod InCastingMethod )
				: ToMixType(InToMixType), NumInputs(InNumInputs), FormatChoosingMethod(InFormatChosingMethod), ChannelMapMonoUpmixMethod(InUpMixMethod), CatCastingMethod(InCastingMethod)
			{}
			FName ToMixType;
			int32 NumInputs = 1;
			EMetasoundMixerFormatChoosingMethod FormatChoosingMethod;
			EMetasoundChannelMapMonoUpmixMethod ChannelMapMonoUpmixMethod;
			EMetasoundCatCastingMethod CatCastingMethod;
		};

		// Linkage.
		const FLazyName FCatMixerOperatorData::OperatorDataTypeName = TEXT("FCatMixerOperatorData");

		const FLazyName CatInputBaseName{"In"};
		const FLazyName GainInputBaseName{ "Gain" };

		static FName MakeCatInputVertexName(const int32 InIndex)
		{
			FName Name = CatInputBaseName;
			Name.SetNumber(InIndex);
			return Name;
		}

		static FName MakeGainInputVertexName(const int32 InIndex)
		{
			FName Name = GainInputBaseName;
			Name.SetNumber(InIndex);
			return Name;
		}

		static FInputDataVertex MakeInputCatVertex(const int32 InIndex, const FName InName, const FString& InFriendlyName)
		{
			const FName InputName = MakeCatInputVertexName(InIndex);
			const FText InputDisplayName = METASOUND_LOCTEXT_FORMAT("In_DisplayName", "{0}", FText::FromString(FString::Printf(TEXT("%s %d"), *InName.ToString(), InIndex)));
			return TInputDataVertex<FChannelAgnosticType>{InputName, FDataVertexMetadata{FText::FromString(*InFriendlyName), InputDisplayName}};
		}

		static FInputDataVertex MakeInputGainVertex(const int32 InIndex, const FName InName, const FString& InFriendlyName)
		{
			const FName InputName = MakeGainInputVertexName(InIndex);
			const FText InputDisplayName = METASOUND_LOCTEXT_FORMAT("In_DisplayName", "{0}", FText::FromString(FString::Printf(TEXT("%s %d"), *InName.ToString(), InIndex)));
			return TInputDataVertex<float>{InputName, FDataVertexMetadata{ FText::FromString(*InFriendlyName), InputDisplayName }, 1.0f};
		}

		static FVertexInterface GetVertexInterface(const int32 InNumInputs, const FName InOutputFormat)
		{
			FInputVertexInterface InputInterface;
			for (int32 i = 0; i < InNumInputs; i++)
			{
				InputInterface.Add(MakeInputCatVertex(i,CatInputBaseName, TEXT("Cat Input") ));
				InputInterface.Add(MakeInputGainVertex(i, GainInputBaseName, TEXT("Gain Input")));
			}

			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(
				FOutputDataVertex(
					METASOUND_GET_PARAM_NAME(OutputCat),
					InOutputFormat,
					METASOUND_GET_PARAM_METADATA(OutputCat),
					EVertexAccessType::Reference)
				);
			
			return FVertexInterface
				{
					MoveTemp(InputInterface),
					MoveTemp(OutputInterface)
				};
		}

		static const Audio::FChannelTypeFamily* FindHighestInputType(const TArray<FChannelAgnosticTypeReadRef>& Inputs)
		{
			if (const FChannelAgnosticTypeReadRef* Found = Algo::MaxElementBy(Inputs, [](const FChannelAgnosticTypeReadRef& i) { return i->NumChannels(); }) )
			{
				return &(*Found)->GetType();
			}
			return nullptr;
		}
		static const Audio::FChannelTypeFamily* FindLowestInputType(const TArray<FChannelAgnosticTypeReadRef>& Inputs)
		{
			if (const FChannelAgnosticTypeReadRef* Found = Algo::MinElementBy(Inputs, [](const FChannelAgnosticTypeReadRef& i) { return i->NumChannels(); }) )
			{
				return &(*Found)->GetType();
			}
			return nullptr;
		}
		static const Audio::FChannelTypeFamily* ChooseOutputFormat(
			const TArray<FChannelAgnosticTypeReadRef>& InOutputs, const Audio::IChannelTypeRegistry& InRegistry, const FCatMixerOperatorData& InOperatorData, const FMetasoundEnvironment& InEnvironment)
		{
			using enum EMetasoundMixerFormatChoosingMethod;
			switch (InOperatorData.FormatChoosingMethod)
			{
			case MetasoundOutput:
			{
#if 0 // This is in another followup CL 
				// Ask for number of channels AudioMixer is configured to use.
				const int32 NumChannels = FMath::Clamp(InEnvironment.GetValue<int32>(Frontend::SourceInterface::Environment::AudioMixerNumChannels)
					, 1, AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

				// Convert that using standard mapping to ChannelMask.
				TOptional<uint32> Mask = Audio::FChannelAgnosticUtils::FindChannelMaskFromAudioMixerNumChannels(NumChannels); 
				if (!ensure(Mask.IsSet()))
				{
					return nullptr;
				}
					
				// Lookup mask in discrete registry. 
				return InRegistry.FindDiscreteChannelFromBitMask(*Mask).Get();
#endif
			}
			case HighestInput:		return FindHighestInputType(InOutputs);
			case LowestInput:		return FindLowestInputType(InOutputs);
			case Custom:			return InRegistry.FindChannel(InOperatorData.ToMixType);
			default:				checkNoEntry();
			}
			return nullptr;
		}
		static const Audio::FChannelTypeFamily* ResolveOutputFormat(
		const TArray<FChannelAgnosticTypeReadRef>& InOutputs, const Audio::IChannelTypeRegistry& InRegistry, const FCatMixerOperatorData& InOperatorData, const FMetasoundEnvironment& InEnvironment)
		{
			if (const Audio::FChannelTypeFamily* Found = ChooseOutputFormat(InOutputs, InRegistry, InOperatorData, InEnvironment ))
			{
				return InRegistry.FindConcreteChannel(Found->GetName());
			}
			return nullptr;
		}
	}
	
	class FCatMixerOperator final : public TExecutableOperator<FCatMixerOperator>
	{
	public:
		FCatMixerOperator(TArray<FFloatReadRef>&& InGains, TArray<FChannelAgnosticTypeReadRef>&& InInputCat, FChannelAgnosticTypeWriteRef&& InOutputCat,
			const FOperatorSettings& InSettings, const CatMixerPrivate::FCatMixerOperatorData& InOpData)
			: Gains(MoveTemp(InGains))
			, Inputs(MoveTemp(InInputCat))
			, Outputs(MoveTemp(InOutputCat))
			, Settings(InSettings)
			, TranscodeScratch(InSettings, Outputs->GetType().GetName())
		{
			PrevGains.Reset(Gains.Num());
			for (FFloatReadRef& Gain : Gains)
			{
				PrevGains.Add(*Gain);
			}
			for  (FChannelAgnosticTypeReadRef& Input : Inputs)
			{
				Transcoders.Emplace(
					Audio::FTranscoderResolver::Resolve(
						Input->GetType(),
				{
							.ToType = Outputs->GetType(),
							.TranscodeMethod = static_cast<Audio::EChannelTranscodeMethod>(InOpData.CatCastingMethod),
							.MixMethod = static_cast<Audio::EChannelMapMonoUpmixMethod>(InOpData.ChannelMapMonoUpmixMethod)
						}));
			}
		}
		virtual ~FCatMixerOperator() override = default;

		static const FNodeClassMetadata& GetNodeInfo()
		{
			static const FName DefaultOutputFormat = TEXT("Cat");
			static const FNodeClassMetadata Metadata = CreateNodeClassMetadata
			(
				FName(TEXT("CAT Mixer Node")),
				METASOUND_LOCTEXT("Metasound_CatMixerNodeDisplayName", "CAT Mixer Node"),
				METASOUND_LOCTEXT("Metasound_CatMixerNodeDescription", "CAT Mixer Node"),
				CatMixerPrivate::GetVertexInterface(1, DefaultOutputFormat)
			);
			return Metadata;
		}
				
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CatMixerPrivate;

			const FCatMixerOperatorData* OperatorData = CastOperatorData<const FCatMixerOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!OperatorData)
			{
				return MakeUnique<FNoOpOperator>();
			}
			
			// inputs.
			TArray<FChannelAgnosticTypeReadRef> Inputs;
			TArray<FFloatReadRef> Gains;
			for (int32 i = 0; i < OperatorData->NumInputs; ++i)
			{
				Inputs.Emplace(InParams.InputData.GetOrCreateDefaultDataReadReference<FChannelAgnosticType>(MakeCatInputVertexName(i), InParams.OperatorSettings));
				Gains.Emplace(InParams.InputData.GetOrCreateDefaultDataReadReference<float>(MakeGainInputVertexName(i), InParams.OperatorSettings));
			}

			// output.
			const Audio::FChannelTypeFamily* Resolved = ResolveOutputFormat(Inputs,Audio::GetChannelRegistry(), *OperatorData, InParams.Environment);
			if (!Resolved)
			{
				return MakeUnique<FNoOpOperator>();
			}
			
			// Make the node.
			return MakeUnique<FCatMixerOperator>(
				MoveTemp(Gains),
				MoveTemp(Inputs),
				FChannelAgnosticTypeWriteRef::CreateNew(InParams.OperatorSettings, Resolved->GetName()),
				InParams.OperatorSettings,
				*OperatorData
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatMixerPrivate;
			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				InOutVertexData.BindReadVertex(MakeCatInputVertexName(i), Inputs[i]);
				InOutVertexData.BindReadVertex(MakeGainInputVertexName(i), Gains[i]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatMixerPrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputCat), Outputs);
		}

		void Reset(const FResetParams&)
		{
			Outputs->Zero();
			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				PrevGains[i] = 0.f;
			}
		}

		static void MixInDiscreteCat(const FChannelAgnosticType& InSrc, FChannelAgnosticType& InDst, const float InStartGain, const float InEndGain)
		{
			// these are the same format now as they've been transcoded.
			check(InSrc.NumChannels() == InDst.NumChannels());
			check(InSrc.IsA(InDst.GetType().GetName()));
			
			const int32 NumChannels = InSrc.NumChannels();
			for (int32 i = 0; i < NumChannels; ++i)
			{
				TArrayView<float> Dst = InDst.GetChannel(i);
				TArrayView<const float> Src = InSrc.GetChannel(i);
				Audio::ArrayMixIn(Src, Dst, InStartGain, InEndGain);
			}
		}
		
		void Execute()
		{
			using namespace Audio;

			// Always zero the output to start.
			Outputs->Zero();

			const int32 NumFrames = Settings.GetNumFramesPerBlock();
			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				const FChannelAgnosticTypeReadRef& Input = Inputs[i]; 
				const float PrevGainValue = PrevGains[i];
				const float GainValue = *Gains[i];
				{
					// Transcode channel into scratch buffer.
					TStackArrayOfPointers<const float> Src = MakeMultiMonoPointersFromView(Input->GetRawMultiMono(), NumFrames, Input->NumChannels());
					TStackArrayOfPointers<float> Dst = MakeMultiMonoPointersFromView(TranscodeScratch.GetRawMultiMono(), NumFrames, Outputs->NumChannels());
					Transcoders[i](Src, Dst, NumFrames);
				}

				// Mix scratch buffer into results. (lerp previous gain and current to avoid zippering).
				MixInDiscreteCat(TranscodeScratch, *Outputs , PrevGainValue, GainValue);

				PrevGains[i] = GainValue;
			}
		}
		
	private:

		TArray<FFloatReadRef> Gains;
		TArray<float> PrevGains;
		TArray<FChannelAgnosticTypeReadRef> Inputs;
		FChannelAgnosticTypeWriteRef Outputs;
		FOperatorSettings Settings;
		TArray<Audio::ChannelAgnosticTranscoder::FTranscoder> Transcoders;
		FChannelAgnosticType TranscodeScratch;

		static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { "CatAudioMixer", InOperatorName, FName() },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{ NodeCategories::Mix },
				{ METASOUND_LOCTEXT("Metasound_AudioMixerKeyword", "Mixer") },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}
	}; // class FCatMixerOperator

	using FCatMixerNode = TNodeFacade<FCatMixerOperator>;

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatMixerNode, FMetaSoundCatMixingNodeConfiguration);
	
} // namespace Metasound

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatMixingNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound;
	using namespace CatMixerPrivate;
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(
		FMetasoundFrontendClassInterface::GenerateClassInterface(
				GetVertexInterface(NumInputs, CustomMixFormat )
				));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatMixingNodeConfiguration::GetOperatorData() const
{
	using namespace Metasound::CatMixerPrivate;
	return MakeShared<FCatMixerOperatorData>(CustomMixFormat, NumInputs, FormatChoosingMethod, ChannelMapMonoUpmixMethod, CatCastingMethod);
}


#undef LOCTEXT_NAMESPACE // "MetasoundStandardNodes_CatMixerNode"
