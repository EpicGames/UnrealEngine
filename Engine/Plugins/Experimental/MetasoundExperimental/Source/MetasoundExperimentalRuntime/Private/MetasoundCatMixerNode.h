// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "MetasoundCatCastingNode.h"
#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"

#include "MetasoundCatMixerNode.generated.h"

UENUM()
enum class EMetasoundMixerFormatChoosingMethod: uint8
{
	HighestInput UMETA(DisplayName = "Highest Num Channels", ToolTip = "Output will be the MAX Channel count of the input channel"),
	LowestInput UMETA(DisplayName = "Lowest Num Channels", ToolTip = "Output will be the MIN Channel count of the input channel"),
	MetasoundOutput UMETA(DisplayName = "Metasound Output", ToolTip = "Output will match the Metasound Output setting"),
	Custom UMETA(DisplayName = "Custom", ToolTip = "Custom mix format"),
};

USTRUCT()
struct FMetaSoundCatMixingNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatMixingNodeConfiguration() = default;

	UPROPERTY(EditAnywhere, Category = General, DisplayName = "Mix Format Method")
	EMetasoundMixerFormatChoosingMethod FormatChoosingMethod = EMetasoundMixerFormatChoosingMethod::HighestInput;

	UPROPERTY(EditAnywhere, Category = General, meta=(EditCondition = "CatCastingMethod == EMetasoundCatCastingMethod::MixUpOrDown", EditConditionHides))
	EMetasoundChannelMapMonoUpmixMethod ChannelMapMonoUpmixMethod = EMetasoundChannelMapMonoUpmixMethod::EqualPower;

	UPROPERTY(EditAnywhere, Category = General)
	EMetasoundCatCastingMethod CatCastingMethod = EMetasoundCatCastingMethod::ChannelDrop;

	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions", EditCondition = "FormatChoosingMethod == EMetasoundMixerFormatChoosingMethod::Custom", EditConditionHides))
	FName CustomMixFormat = TEXT("Cat");

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin=1, ClampMax=100))
	int32 NumInputs = 1;
	
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};
