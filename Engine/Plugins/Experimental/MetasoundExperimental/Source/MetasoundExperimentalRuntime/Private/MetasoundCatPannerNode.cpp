// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundCatPannerNode.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundVertex.h"
#include "DSP/Vbap.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatPannerNode"

namespace Metasound
{
	class FChannelAgnosticType;
	struct FBuildOperatorParams;
	class FNodeFacade;

	// Mapping function to convert from user facing normalized values. (0-2) 
	// (0-1) Upper Hemisphere (Clockwise Left to Right) -90 to 90  
	// (1-2) Lower Hemisphere (Clockwise Right to Left) 90 to -90 (sign flip through -180/180)
	float NormalizedAzimuthToDegrees(const float InNormalizedAzimuth)
	{
		float WrappedAzimuth = FMath::Fmod(InNormalizedAzimuth, 2.0f);
		if (WrappedAzimuth < 0.0f)
		{
			WrappedAzimuth += 2.0f;
		}
			
		// Shift so 0.5 is forward (0 degrees), and scale 2.0 to (360 degrees)
		float Degrees = (WrappedAzimuth - 0.5f) * 180.0f;

		// Wrap to [-180, 180)
		Degrees = FMath::Fmod(Degrees + 180.0f, 360.0f);
		if (Degrees < 0.0f)
		{
			Degrees += 360.0f;
		}
		Degrees -= 180.0f;
			
		return Degrees;
	}

	namespace CatPannerPrivate
	{
		METASOUND_PARAM(InputAudio,			"Input",	"Input audio to pan.");
		METASOUND_PARAM(InputAzimuth,		"Azimuth",	"The normalized azimuthal angle to pan the sound in range [0.0, 2.0). (0.0 is left, 0.5 is center, 1.0 is right, 1.5 is behind.");
		METASOUND_PARAM(OutputToCat,		"Output",	"Panned audio.");
		
		class FCatPannerOperatorData final : public TOperatorData<FCatPannerOperatorData>
		{
		public:
			static const FLazyName OperatorDataTypeName;

			explicit FCatPannerOperatorData(const FName& InPanToTypeName, ECatPannerMethod InPannerMethod)
				: PanToTypeName(InPanToTypeName)
				, PannerMethod(InPannerMethod)
			{}
			const FName& GetPanToType() const { return PanToTypeName; }
			ECatPannerMethod GetPannerMethod() const { return PannerMethod; }
		private:
			FName PanToTypeName;
			ECatPannerMethod PannerMethod;
		};

		// Linkage.
		const FLazyName FCatPannerOperatorData::OperatorDataTypeName = TEXT("FCatPannerOperatorData");

	}
	
	class FCatPannerOperator final : public TExecutableOperator<FCatPannerOperator>
	{
	public:
		using FDiscreteChannelTypeFamily = Audio::FDiscreteChannelTypeFamily;
		using FCatPannerOperatorData = CatPannerPrivate::FCatPannerOperatorData;

		FCatPannerOperator(
				const FBuildOperatorParams& InParams, 
				FAudioBufferReadRef&& InAudio, 
				FFloatReadRef&& InAzimuth,
				FDiscreteChannelAgnosticTypeWriteRef&& InCatOutput,
				const Audio::IDiscretePanner& InPanner,
				const FCatPannerOperatorData* InConfigData)
			: InputAudio(MoveTemp(InAudio))
			, InputAzimuth(MoveTemp(InAzimuth))
			, PannerMethod(InConfigData->GetPannerMethod())
			, OutputPannedAudio(MoveTemp(InCatOutput))
			, Settings(InParams.OperatorSettings)
			, Panner(InPanner)
		{
			LastUpdateGainsPerChannel.SetNum(InCatOutput->NumChannels());
			for (int32 i = 0; i < InCatOutput->NumChannels(); ++i)
			{
				LastUpdateGainsPerChannel[i] = -1.f;
			}
		}
		
		virtual ~FCatPannerOperator() override = default;

		static FVertexInterface GetInterface(const FName OutputFormat)
		{
			using namespace CatPannerPrivate;
			// inputs
			FInputVertexInterface InputInterface;
			InputInterface.Add(TInputDataVertex<FAudioBuffer>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAudio)));
			InputInterface.Add(TInputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(InputAzimuth), 0.0f));

			// outputs
			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(
				FOutputDataVertex(
					METASOUND_GET_PARAM_NAME(OutputToCat),
					OutputFormat,
					METASOUND_GET_PARAM_METADATA(OutputToCat),
					EVertexAccessType::Reference)
				);

			return FVertexInterface(InputInterface, OutputInterface);
		}
			
		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace CatPannerPrivate;

			const FCatPannerOperatorData* ConfigData = CastOperatorData<const FCatPannerOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!ConfigData)
			{
				return MakeUnique<FNoOpOperator>();
			}
			const FName PanToName = ConfigData->GetPanToType();
			const Audio::FChannelTypeFamily* ConcretePanToType = Audio::GetChannelRegistry().FindConcreteChannel(PanToName);
			if (!ConcretePanToType)
			{
				return MakeUnique<FNoOpOperator>();
			}

			TDataReadReference<FAudioBuffer> InputAudio = 
				InParams.InputData.GetOrCreateDefaultDataReadReference<FAudioBuffer>(METASOUND_GET_PARAM_NAME(InputAudio), InParams.OperatorSettings);
			
			TDataReadReference<float> InputAzimuth =
				InParams.InputData.GetOrCreateDefaultDataReadReference<float>(METASOUND_GET_PARAM_NAME(InputAzimuth), InParams.OperatorSettings);

			// Make Discrete CAT output.
			// These are defined in data so we must use "DerivedAs" 
			TDataWriteReference<FDiscreteChannelAgnosticType> CatPin =
				FDiscreteChannelAgnosticTypeWriteRef::CreateNewDerivedAs<FDiscreteChannelAgnosticType>(
					ConcretePanToType->GetName(),
					Metasound::GetMetasoundDataTypeId<FDiscreteChannelAgnosticType>(),
					InParams.OperatorSettings,
					ConcretePanToType->GetName()
				);

			// Ask target format for a panner. 
			const Audio::FChannelTypeFamily::FGetPannerParams Params;
			const Audio::IDiscretePanner* InPanner = CatPin->GetType().GetPanner(Params);
			if (!InPanner)
			{
				return MakeUnique<FNoOpOperator>();
			}
			
			return MakeUnique<FCatPannerOperator>(
				InParams,
				MoveTemp(InputAudio),
				MoveTemp(InputAzimuth),
				MoveTemp(CatPin),
				*InPanner,
				ConfigData);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatPannerPrivate;
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAudio) , InputAudio);
			InOutVertexData.BindReadVertex(METASOUND_GET_PARAM_NAME(InputAzimuth), InputAzimuth);
		}
		
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace CatPannerPrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutputToCat), OutputPannedAudio);
		}

		void Reset(const FResetParams& InParams)
		{
			Execute();
		}

		void ApplyGains(const Audio::IDiscretePanner::FOutputParams& InOutput)
		{
			const FDiscreteChannelTypeFamily& Discrete = *OutputPannedAudio->GetType().Cast<FDiscreteChannelTypeFamily>();

			const int32 InputSampleCount = InputAudio->Num();

			auto ApplyGain = [](float& Prev, const float Current, TArrayView<const float> Src, TArrayView<float> Dst)
				{
					if (FMath::IsNearlyZero(Current))
					{
						// Nothing so far and zero gain. No nothing.
						if (Prev < 0.f) 
						{
							return;
						}

						// Zero start+end. (we can zero).
						if ( FMath::IsNearlyZero(Prev)) 
						{
							FMemory::Memzero(Dst.GetData(), Dst.NumBytes());
							return;
						}
					}

					// Special case if prev is 100% and current is 100%, just copy source out.
					if (FMath::IsNearlyEqual(Prev,1.0f) && FMath::IsNearlyEqual(Current,1.0f))
					{
						FMemory::Memcpy(Dst.GetData(), Src.GetData(), Src.NumBytes());
					}
					else
					{
						// Clamp previous >= 0
						const float PrevClamped = FMath::Max(0,Prev);
						
						// Requires some kind of fade...
						Audio::ArrayFade(Src, PrevClamped, Current, Dst);
					}

					// Remember the gain as previous for next round.
					Prev = Current;
				};

			
			const int32 NumChannels = OutputPannedAudio->NumChannels();
			for (int32 i = 0; i < NumChannels; ++i)
			{
				// If this channel is active in the pan results.
				const Audio::IDiscretePanner::FPanResult* ActiveSpeaker = InOutput.Results.FindByPredicate(
					[i](const Audio::IDiscretePanner::FPanResult& PanResult) -> bool
						{
							check(PanResult.ChannelIndex != INDEX_NONE);
							return PanResult.ChannelIndex == i;
						});

				const float TargetGain = ActiveSpeaker ? ActiveSpeaker->Gain : 0.f;
				ApplyGain(LastUpdateGainsPerChannel[i], TargetGain, *InputAudio, OutputPannedAudio->GetChannel(i));
			}
		}
	
		
		void Execute()
		{
			// 0-1 (upper hemisphere clockwise  Left+Right) -90 to 90 (9pm - 3pm)
			// 1-2 (lower hemisphere clockwise Right-Left) 90 to -90  (3pm - 9pm)

			const float AzimuthDegrees = NormalizedAzimuthToDegrees(*InputAzimuth);

			// Zero output.
			// OutputPannedAudio->Zero();
			
			Audio::IDiscretePanner::FOutputParams OutputParam;
			Panner.ComputeGains(
				{
					.AzimuthDegrees = AzimuthDegrees,
					.ElevationDegrees = 0.f,
					.bAllowAzimuthMirroring = true
				},
				OutputParam
			);

			// Apply gains.
			ApplyGains(OutputParam);
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace CatPannerPrivate;
			static const FName DefaultFormat(TEXT("Cat:Stereo2Dot0"));
			static const FText MissingText = LOCTEXT("ExampleCatPannerPromptIfMissing", "Enable the MetaSoundExperimental Plugin");
			static const FText NodeNameText = LOCTEXT("CatPannerNodeName", "Azimuth Panner");
			static const FText NodeDescText = LOCTEXT("CatPannerNodeNameDescription", "A Node that does an azimuthal pan to an output CAT type");
			
		return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "CatPannerOperator", "" },
				1, // Major version
				0, // Minor version
				NodeNameText,	
				NodeDescText,	
				TEXT("UE"), // Author
				MissingText, // Prompt if missing
				GetInterface(DefaultFormat),
				{}
			};
		}
	private:
		FAudioBufferReadRef InputAudio;
		FFloatReadRef InputAzimuth;
		ECatPannerMethod PannerMethod;
		FDiscreteChannelAgnosticTypeWriteRef OutputPannedAudio;
		FOperatorSettings Settings;
		const Audio::IDiscretePanner& Panner;
		TArray<float> LastUpdateGainsPerChannel;
		TMap<ESpeakerShortNames, int32> SpeakerMapping;
	}; 
		
	using FCatPannerNode = TNodeFacade<FCatPannerOperator>;

	// register node config.	
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatPannerNode, FMetaSoundCatPannerNodeConfiguration);
} // namespace Metasound

TArray<FPropertyTextFName> UMetasoundCatAzimuthalChannelOptionsHelper::GetAzimuthalChannelOptions()
{
	using namespace Audio;
	TArray<TSharedRef<const FChannelTypeFamily>> AllFormats = Audio::GetChannelRegistry().GetAllChannelFormats();
	TArray<FPropertyTextFName> FormatsOptions;
	for (TSharedRef Format : AllFormats)
	{
		if (const FDiscreteChannelTypeFamily* Discrete = Format->Cast<FDiscreteChannelTypeFamily>())
		{
			if (Discrete->GetPanner() != nullptr) // If there's a panner defined, we can use it.
			{
				FormatsOptions.Emplace(Format->GetName(), FText::FromString(Format->GetFriendlyName()));
			}
		}
	}
	return FormatsOptions;
}

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatPannerNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(Metasound::FCatPannerOperator::GetInterface(this->PanToType)));		// Assume this is concrete type.
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatPannerNodeConfiguration::GetOperatorData() const
{
	return MakeShared<Metasound::CatPannerPrivate::FCatPannerOperatorData>(PanToType, PanningMethod);
}

#undef LOCTEXT_NAMESPACE // 
