// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetasoundCatTestingNode.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundFacade.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundVertex.h"
#include "TypeFamily/ChannelTypeFamily.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatTestingNode"

namespace Metasound
{
	namespace FCatTestingNodePrivate
	{
		METASOUND_PARAM(InputNumChannels, "NumChannels", "Number of cat channels to generate");
		METASOUND_PARAM(OutputCat, "Output", "Channel Agnostic Output");

		FVertexInterface MakeClassInterface(const FName Format)
		{
			const Audio::FChannelTypeFamily* Found = Audio::GetChannelRegistry().FindConcreteChannel(Format);
			if (!Found)
			{
				return {};
			}
			FInputVertexInterface InputInterface;
			FOutputVertexInterface OutputInterface;
			OutputInterface.Add(TOutputDataVertex<FChannelAgnosticType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputCat)));
			return{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
		
		class FCatTestingOperatorData final : public TOperatorData<FCatTestingOperatorData>
		{
		public:
			// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
			// that the downcast is valid.
			static const FLazyName OperatorDataTypeName;

			explicit FCatTestingOperatorData(const FName& InToTypeName)
				: ToTypeName(InToTypeName)
			{}
			const FName& GetToType() const
			{
				return ToTypeName;
			}
		private:
			FName ToTypeName;
		};

		// Linkage.
		const FLazyName FCatTestingOperatorData::OperatorDataTypeName = TEXT("FCatTestingOperatorData");
	}

	class FCatTestingOperator final : public TExecutableOperator<FCatTestingOperator>
	{
	public:
		FCatTestingOperator(const FBuildOperatorParams& InParams, const FName InConcreteFormat)
			: SampleRate(InParams.OperatorSettings.GetSampleRate())
			, NumFramesPerBlock(InParams.OperatorSettings.GetNumFramesPerBlock())
			, Outputs(FChannelAgnosticTypeWriteRef::CreateNew(InParams.OperatorSettings, InConcreteFormat))	
			, Settings(InParams.OperatorSettings)
		{}
		virtual ~FCatTestingOperator() override = default;

		static const FVertexInterface& DeclareVertexInterface()
		{
			auto CreateDefaultInterface = []()-> FVertexInterface
			{
				using namespace FCatTestingNodePrivate;
				const FInputVertexInterface InputInterface;
				const FOutputVertexInterface OutputInterface
				{
					TOutputDataVertex<FChannelAgnosticType>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutputCat))
				};
				return FVertexInterface(InputInterface, OutputInterface);
			};

			static const FVertexInterface DefaultInterface = CreateDefaultInterface();
			return DefaultInterface;
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace FCatTestingNodePrivate;

			const FCatTestingOperatorData* TestingOperatorData = CastOperatorData<const FCatTestingOperatorData>(InParams.Node.GetOperatorData().Get());
			if (!TestingOperatorData)
			{
				return TUniquePtr<FNoOpOperator>();
			}
			const Audio::FChannelTypeFamily* Concrete = Audio::GetChannelRegistry().FindConcreteChannel(TestingOperatorData->GetToType());
			if (!Concrete)
			{
				return TUniquePtr<FNoOpOperator>();
			}
			
			return MakeUnique<FCatTestingOperator>(
				InParams,
				Concrete->GetName()
			);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			InOutVertexData.BindWriteVertex(FCatTestingNodePrivate::OutputCatName, Outputs);
		}

		void Reset(const FResetParams&)
		{
			Execute();
		}

		float MakeTone(TArrayView<float> InBuffer, const int32 InFreq) const
		{
			float* Dst = InBuffer.GetData();
			const float PhaseStep = SampleRate / InFreq;;
			float Phase = 0.f;

			for (int32 Remaining = InBuffer.Num(); Remaining > 0; Remaining--)
			{
				*Dst++ = FMath::Sin(Phase * UE_TWO_PI);
				Phase += PhaseStep;
				Phase = Phase - static_cast<int32>(Phase);
			}
			return Phase;
		}
		
		void Execute()
		{
			check(NumFramesPerBlock == Outputs->NumFrames());

			// Create N channels of increasing frequency tones.
			const int32 NumChannels = Outputs->NumChannels();;
			for (int32 i = 0, Freq = 110; i < NumChannels; ++i, Freq *=2)
			{
				MakeTone(Outputs->GetChannel(i),Freq);
			}
		}

		static const FNodeClassMetadata& GetNodeInfo()
		{
			// used if NumChannels > 2
			auto CreateNodeClassMetadataMultiChan = []() -> FNodeClassMetadata
			{
				const FName OperatorName = TEXT("CAT Testing Node");
				const FText NodeDisplayName = METASOUND_LOCTEXT("CatTestingNodeDisplayName", "CAT Testing Node");
				const FText NodeDescription = METASOUND_LOCTEXT("CatTestingNodeDescription", "CAT Testing Node");
				const FVertexInterface NodeInterface =  DeclareVertexInterface();
				return  CreateNodeClassMetadata(OperatorName, NodeDisplayName, NodeDescription, NodeInterface);
			};
			static const FNodeClassMetadata Metadata = CreateNodeClassMetadataMultiChan();
			return Metadata;
		}


	private:
		float SampleRate = 0.f;
		int32 NumFramesPerBlock = 0;
		FChannelAgnosticTypeWriteRef Outputs;
		FOperatorSettings Settings;

		static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
		{
			FNodeClassMetadata Metadata
			{
				FNodeClassName { StandardNodes::Namespace, InOperatorName, StandardNodes::AudioVariant },
				1, // Major Version
				0, // Minor Version
				InDisplayName,
				InDescription,
				PluginAuthor,
				PluginNodeMissingPrompt,
				InDefaultInterface,
				{},
				{ METASOUND_LOCTEXT("Metasound_CatTestingNode", "Cat Testing") },
				FNodeDisplayStyle{}
			};

			return Metadata;
		}
	}; // class FCatMixerOperator

	using FCatTestingNode = TNodeFacade<FCatTestingOperator>;
	// Disabled for now.
	//METASOUND_REGISTER_NODE_AND_CONFIGURATION(FCatTestingNode, FMetaSoundCatTestingNodeConfiguration);


} // namespace Metasound

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundCatTestingNodeConfiguration::OverrideDefaultInterface(
	const FMetasoundFrontendClass& InNodeClass) const
{
	return  TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(Metasound::FCatTestingNodePrivate::MakeClassInterface(ToType)));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundCatTestingNodeConfiguration::GetOperatorData() const
{
	return MakeShared<Metasound::FCatTestingNodePrivate::FCatTestingOperatorData>(ToType);
}

#undef LOCTEXT_NAMESPACE // "MetasoundStandardNodes_CatMixerNode"
