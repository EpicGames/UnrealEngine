// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"

#include "MetasoundCatTestingNode.generated.h"

USTRUCT()
struct FMetaSoundCatTestingNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatTestingNodeConfiguration() =  default;

	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatCastingOptionsHelper.GetCastingOptions"))
	FName ToType = TEXT("Mono");

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};
