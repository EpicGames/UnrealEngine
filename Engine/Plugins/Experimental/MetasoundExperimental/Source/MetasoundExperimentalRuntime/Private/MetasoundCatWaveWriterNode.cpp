// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/SimpleWaveWriter.h"
#include "ChannelAgnostic/ChannelAgnosticType.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "NumberedFileCache.h"
#include "HAL/FileManager.h"
#include "MetasoundBuildError.h"
#include "MetasoundChannelAgnosticType.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundStandardNodesCategories.h"
#include "MetasoundStandardNodesNames.h"
#include "MetasoundVertex.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#define LOCTEXT_NAMESPACE "MetasoundStandardNodes_CatWaveWriterNode"

namespace Metasound
{
	namespace Experimental
	{
		namespace WaveWriterVertexNames
		{
			METASOUND_PARAM(InEnabledPin, "Enabled", "If this wave writer is enabled or not.");
			METASOUND_PARAM(InFilenamePrefixPin, "Filename Prefix", "Filename Prefix of file you are writing.");
			METASOUND_PARAM(InCatPin, "InputCat", "Channel Agnostic Input");
		}

		class FFileWriteError final : public FBuildErrorBase
		{
		public:
			static const FName ErrorType;

			virtual ~FFileWriteError() override = default;

			FFileWriteError(const FNode& InNode, const FString& InFilename)
	#if WITH_EDITOR
				: FBuildErrorBase(ErrorType, METASOUND_LOCTEXT_FORMAT("MetasoundFileWriterErrorDescription", "File Writer Error while trying to write '{0}'", FText::FromString(InFilename)))
	#else 
				: FBuildErrorBase(ErrorType, FText::GetEmpty())
	#endif // WITH_EDITOR
			{
				AddNode(InNode);
			}
		};
		const FName FFileWriteError::ErrorType = FName(TEXT("MetasoundFileWriterError"));

		namespace WaveWriterOperatorPrivate
		{
			using namespace Audio;
			
			// Need to keep this outside the template so there's only 1
			TSharedPtr<FNumberedFileCache> GetNameCache()
			{
				static const TCHAR* WaveExt = TEXT(".wav");

				// Build cache of numbered files (do this once only).
				static TSharedPtr<FNumberedFileCache> NumberedFileCacheSP = MakeShared<FNumberedFileCache>(*FPaths::AudioCaptureDir(), WaveExt, IFileManager::Get());
				return NumberedFileCacheSP;
			}

			static const FString& GetDefaultFileName()
			{
				static const FString DefaultFileName = TEXT("Output");
				return DefaultFileName;
			}
		}

		class FCatWaveWriterOperator final : public TExecutableOperator<FCatWaveWriterOperator>
		{
		public:
			FCatWaveWriterOperator(const FBuildOperatorParams& InParams, FChannelAgnosticTypeReadRef&& InAudioBuffers, FBoolReadRef&& InEnabled, const TSharedPtr<FNumberedFileCache, ESPMode::ThreadSafe>& InNumberedFileCache, FStringReadRef&& InFilenamePrefix)
				: AudioInputs{MoveTemp(InAudioBuffers)}
				, Enabled{MoveTemp(InEnabled)}
				, NumberedFileCacheSP{InNumberedFileCache}
				, FileNamePrefix{MoveTemp(InFilenamePrefix)}
				, SampleRate{InParams.OperatorSettings.GetSampleRate()}
				, NumInputFrames{InParams.OperatorSettings.GetNumFramesPerBlock()}
			{
				// Now we have an input, we can ask how many channel it has.
				NumInputChannels = AudioInputs->NumChannels();
				InterleaveBuffer.SetNumZeroed(NumInputChannels * NumInputFrames);
			}

			virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override {}
			virtual void BindOutputs(FOutputVertexInterfaceData&) override {}

			static const FVertexInterface& DeclareVertexInterface()
			{
				auto CreateDefaultInterface = []()-> FVertexInterface
				{
					using namespace WaveWriterOperatorPrivate;
					using namespace WaveWriterVertexNames;

					// inputs
					const FInputVertexInterface InputInterface(
						TInputDataVertex<FString>(METASOUND_GET_PARAM_NAME_AND_METADATA(InFilenamePrefixPin), GetDefaultFileName()),
						TInputDataVertex<bool>(METASOUND_GET_PARAM_NAME_AND_METADATA(InEnabledPin), true),
						TInputDataVertex<FChannelAgnosticType>(METASOUND_GET_PARAM_NAME_AND_METADATA(InCatPin))
					);
					const FOutputVertexInterface OutputInterface;
					return FVertexInterface(InputInterface, OutputInterface);
				};

				static const FVertexInterface DefaultInterface = CreateDefaultInterface();
				return DefaultInterface;
			}

			static const FNodeClassMetadata& GetNodeInfo()
			{
				static const FNodeClassMetadata Metadata = CreateNodeClassMetadata
				(
					FName(TEXT("Cat Wave Writer")),
					METASOUND_LOCTEXT("Metasound_CatWaveWriterNodeMultiChannelDisplayName", "Wave Writer Channel Agnostic"),
					METASOUND_LOCTEXT("Metasound_CatWaveWriterNodeMultiDescription", "Write a CAT audio signal to disk"),
					DeclareVertexInterface()
				);
				return Metadata;
			}

			static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults);
			
			void Execute()
			{
				// Enabled and wasn't before? Enable.
				if (!bIsEnabled && *Enabled)
				{
					Enable();
				}
				// Disabled but currently Enabled? Disable.
				else if (bIsEnabled && !*Enabled)
				{
					Disable();
				}

				// If we have a valid writer and enabled.
				if (Writer && *Enabled && NumInputChannels > 0)
				{
					Audio::FChannelAgnosticUtils::Interleave(*AudioInputs, InterleaveBuffer);
					Writer->Write(InterleaveBuffer);
				}
			}

			void Reset(const IOperator::FResetParams& InParams)
			{
				if (bIsEnabled)
				{
					Disable();
				}
			}

		protected:
			static FNodeClassMetadata CreateNodeClassMetadata(const FName& InOperatorName, const FText& InDisplayName, const FText& InDescription, const FVertexInterface& InDefaultInterface)
			{
				FNodeClassMetadata Metadata
				{
					FNodeClassName { StandardNodes::Namespace, InOperatorName, StandardNodes::AudioVariant },
					1, // Major Version
					1, // Minor Version
					InDisplayName,
					InDescription,
					PluginAuthor,
					PluginNodeMissingPrompt,
					InDefaultInterface,
					{ NodeCategories::Io },
					{ METASOUND_LOCTEXT("Metasound_AudioMixerKeyword", "Writer") },
					FNodeDisplayStyle{}
				};
				return Metadata;
			}

			void Enable()
			{
				if (ensure(!bIsEnabled) && NumInputChannels > 0)
				{
					bIsEnabled = true;
					const FString Filename = NumberedFileCacheSP->GenerateNextNumberedFilename(*FileNamePrefix);
					if (TUniquePtr<FArchive> Stream {IFileManager::Get().CreateFileWriter(*Filename, IO_WRITE) }; Stream.IsValid() )
					{ 					
						Writer = MakeUnique<Audio::FSimpleWaveWriter>(MoveTemp(Stream), SampleRate, NumInputChannels, true);
					}
				}
			}
			void Disable()
			{
				if (ensure(bIsEnabled))
				{
					bIsEnabled = false;
					Writer.Reset();
					NumInputChannels = 0;
				}
			}

			FChannelAgnosticTypeReadRef AudioInputs;
			TArray<float> InterleaveBuffer;
			FBoolReadRef Enabled;
			TUniquePtr<Audio::FSimpleWaveWriter> Writer;
			TSharedPtr<FNumberedFileCache> NumberedFileCacheSP;
			FStringReadRef FileNamePrefix;
			float SampleRate = 0.f;
			int32 NumInputChannels = 0;
			int32 NumInputFrames = 0;
			bool bIsEnabled = false;
		};

		TUniquePtr<IOperator> FCatWaveWriterOperator::CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace WaveWriterOperatorPrivate;
			using namespace WaveWriterVertexNames;

			const FOperatorSettings& Settings = InParams.OperatorSettings;
			const FInputVertexInterfaceData& InputData = InParams.InputData;

			return MakeUnique<FCatWaveWriterOperator>(
				InParams,
				InputData.GetOrCreateDefaultDataReadReference<FChannelAgnosticType>(METASOUND_GET_PARAM_NAME(InCatPin),  Settings),
				InputData.GetOrCreateDefaultDataReadReference<bool>(METASOUND_GET_PARAM_NAME(InEnabledPin), Settings),
				GetNameCache(),
				InputData.GetOrCreateDefaultDataReadReference<FString>(METASOUND_GET_PARAM_NAME(InFilenamePrefixPin), Settings)
			);
		}

		using FCatWaveWriterNode = TNodeFacade<FCatWaveWriterOperator>;
		METASOUND_REGISTER_NODE(FCatWaveWriterNode);
	}// namespace Experimental

} // namespace Metasounds

#undef LOCTEXT_NAMESPACE
