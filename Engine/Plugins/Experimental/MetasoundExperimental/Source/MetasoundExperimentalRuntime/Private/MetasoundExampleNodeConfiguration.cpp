// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExampleNodeConfiguration.h" 

#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

#include "MetasoundDataFactory.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExampleNodeConfiguration.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundTrigger.h"


#define LOCTEXT_NAMESPACE "MetasoundExperimentalRuntime"

namespace Metasound::Experimental
{
	namespace ExampleNodeConfigurationPrivate
	{
		const FLazyName InputBaseName{"In"};
		const FText InputTooltip = METASOUND_LOCTEXT("In_ToolTip", "A trigger");

		FName MakeInputVertexName(int32 InIndex)
		{
			FName Name = InputBaseName;
			Name.SetNumber(InIndex);
			return Name;
		}

		FInputDataVertex MakeInputDataVertex(int32 InIndex)
		{
			FName InputName = MakeInputVertexName(InIndex);
			const FText InputDisplayName = METASOUND_LOCTEXT_FORMAT("In_DisplayName", "In {0}", InIndex);
			return TInputDataVertex<FTrigger>{InputName, FDataVertexMetadata{InputTooltip, InputDisplayName}};
		}

		FLazyName OutputBaseName{"Out"};
		const FText OutputTooltip = METASOUND_LOCTEXT("Out_ToolTip", "A trigger");

		FName MakeOutputVertexName(int32 OutOutdex)
		{
			FName Name = OutputBaseName;
			Name.SetNumber(OutOutdex);
			return Name;
		}

		FOutputDataVertex MakeOutputDataVertex(int32 OutOutdex)
		{
			FName OutputName = MakeOutputVertexName(OutOutdex);
			const FText OutputDisplayName = METASOUND_LOCTEXT_FORMAT("Out_DisplayName", "Out {0}", OutOutdex);
			return TOutputDataVertex<FTrigger>{OutputName, FDataVertexMetadata{OutputTooltip, OutputDisplayName}};
		}


		// Create the node's vertex interface based upon the number of inputs and outputs
		// desired. 
		FVertexInterface GetVertexInterface(int32 InNumInputs, int32 InNumOutputs)
		{
			FInputVertexInterface InputInterface;
			for (int32 i = 0; i < InNumInputs; i++)
			{
				InputInterface.Add(MakeInputDataVertex(i));
			}

			FOutputVertexInterface OutputInterface;
			for (int32 i = 0; i < InNumOutputs; i++)
			{
				OutputInterface.Add(MakeOutputDataVertex(i));
			}

			return FVertexInterface
				{
					MoveTemp(InputInterface),
					MoveTemp(OutputInterface)
				};
		}

		METASOUND_PARAM(OutFloat, "Out", "Float output");
	}

	/** To send data from a FMetaSoundFrontendNodeConfiguration to an IOperator, it should
	 * be encapsulated in the form of a IOperatorData.  
	 *
	 * The use of the TOperatorData provides some safety mechanisms for downcasting node configurations.
	 */
	class FExampleOperatorData : public TOperatorData<FExampleOperatorData>
	{
	public:
		// The OperatorDataTypeName is used when downcasting an IOperatorData to ensure
		// that the downcast is valid.
		static const FLazyName OperatorDataTypeName;

		FExampleOperatorData(const FString& InString)
		: String(InString)
		{
		}

		const FString& GetString() const
		{
			return String;
		}

	private:

		FString String;
	};

	const FLazyName FExampleOperatorData::OperatorDataTypeName = "ExperimentalExampleOperatorData";

	class FExampleConfigurableOperator : public TExecutableOperator<FExampleConfigurableOperator>
	{
	public:

		FExampleConfigurableOperator(const FString& InString, TArray<TDataReadReference<FTrigger>> InInputTriggers, TArray<TDataWriteReference<FTrigger>> InOutputTriggers)
		: InputTriggers(MoveTemp(InInputTriggers))
		, OutputTriggers(MoveTemp(InOutputTriggers))
		{
			UE_LOGF(LogMetaSound, Display, "Did the configurable string make it: %ls", *InString);
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExampleNodeConfigurationPrivate;

			for (int32 i = 0; i < InputTriggers.Num(); i++)
			{
				InOutVertexData.BindReadVertex(MakeInputVertexName(i), InputTriggers[i]);
			}
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExampleNodeConfigurationPrivate;

			for (int32 i = 0; i < OutputTriggers.Num(); i++)
			{
				InOutVertexData.BindReadVertex(MakeOutputVertexName(i), OutputTriggers[i]);
			}
		}

		void Execute()
		{
			// This node randomly rearranges all input triggers across all the 
			// output triggers.
			if (OutputTriggers.Num() > 0)
			{
				for (const TDataWriteReference<FTrigger>& OutputTrigger : OutputTriggers)
				{
					OutputTrigger->AdvanceBlock();
				}
				for (const TDataReadReference<FTrigger>& InputTrigger : InputTriggers)
				{
					InputTrigger->ExecuteBlock(
							[](int32, int32) {},
							[this](int32 InStartFrame, int32 InEndFrame)
							{
								int32 OutputIndex = FMath::RandRange(0, OutputTriggers.Num() - 1);
								OutputTriggers[OutputIndex]->TriggerFrame(InStartFrame);
							}
					);
				}
			}
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			for (const TDataWriteReference<FTrigger>& OutputTrigger : OutputTriggers)
			{
				OutputTrigger->Reset();
			}
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace ExampleNodeConfigurationPrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "ConfigurableOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("ExampleConfigurableNodeName", "A Configurable Node"),	
				METASOUND_LOCTEXT("ExampleConfigurableNodeDescription", "A Node which shows how to make a configurable node for yourself."),	
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("ExampleConfigurablePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				ExampleNodeConfigurationPrivate::GetVertexInterface(1 /* default num inputs */, 1 /* default num outputs */),
				{}
			};
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) 
		{
			using namespace ExampleNodeConfigurationPrivate;

			// If your node configuration contains data that needs to be accessed by
			// an operator, it can be accessed in the CreateOperator call. The 
			// TSharedPtr<const IOperatorData> could even be stored on the IOperator. 
			//
			// In this example, the function `GetOperatorDataAs<>` safely downcasts
			// the TSharedPtr<const IOperatorData> retrieved from the node. 
			FString ConfiguredString = TEXT("Nope");
			if (const FExampleOperatorData* ExampleConfig = CastOperatorData<const FExampleOperatorData>(InParams.Node.GetOperatorData().Get()))
			{
				ConfiguredString = ExampleConfig->GetString();
			}

			// If the node configuration overrides the default class interface, 
			// the node's vertex interface will reflect the override. The vertex
			// interface can be queried to see which inputs and outputs exist.
			//
			// For more complex scenarios, developers may want to pass other data
			// through the IOperatorData. Alternatively, this node could have put
			// the number of inputs and outputs into the IOperator data. 
			const FVertexInterface& NodeInterface = InParams.Node.GetVertexInterface();

			// Build the correct data references based upon what vertices exist
			// on the NodeInterface
			TArray<TDataReadReference<FTrigger>> InputTriggers;
			int32 InputIndex = 0;
			while (true)
			{
				FVertexName VertexName = MakeInputVertexName(InputIndex);
				if (NodeInterface.ContainsInputVertex(VertexName))
				{
					InputTriggers.Add(InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(VertexName, InParams.OperatorSettings));
				}
				else
				{
					break;
				}
				InputIndex++;
			}

			TArray<TDataWriteReference<FTrigger>> OutputTriggers;
			int32 OutputIndex = 0;
			while (true)
			{
				FVertexName VertexName = MakeOutputVertexName(OutputIndex);
				if (NodeInterface.ContainsOutputVertex(VertexName))
				{
					OutputTriggers.Add(TDataWriteReferenceFactory<FTrigger>::CreateExplicitArgs(InParams.OperatorSettings));
				}
				else
				{
					break;
				}
				OutputIndex++;
			}


			return MakeUnique<FExampleConfigurableOperator>(
					ConfiguredString,
					MoveTemp(InputTriggers),
					MoveTemp(OutputTriggers)
					);
		}

	private:
		TArray<TDataReadReference<FTrigger>> InputTriggers;
		TArray<TDataWriteReference<FTrigger>> OutputTriggers;
	};

	using FExampleConfigurableNode = TNodeFacade<FExampleConfigurableOperator>;

	// The node extension must be registered along with the node. 
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FExampleConfigurableNode, FMetaSoundExperimentalExampleNodeConfiguration);

	// FMetaSoundWidgetExampleNodeConfiguration
	const FLazyName FWidgetExampleOperatorData::OperatorDataTypeName = "ExperimentalWidgetExampleOperatorData";

	class FWidgetExampleConfigurableOperator : public TExecutableOperator<FWidgetExampleConfigurableOperator>
	{
	public:
		FWidgetExampleConfigurableOperator(const TSharedPtr<const FWidgetExampleOperatorData>& InOperatorData)
			: OperatorData(InOperatorData)
			, FloatOut(FFloatWriteRef::CreateNew())
		{
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExampleNodeConfigurationPrivate;
			InOutVertexData.BindWriteVertex(METASOUND_GET_PARAM_NAME(OutFloat), FloatOut);
		}

		void Execute()
		{
			*FloatOut = OperatorData->MyFloat;
		}
		
		static const FVertexInterface& GetVertexInterface()
		{
			using namespace ExampleNodeConfigurationPrivate;
			static const FVertexInterface Interface(
				FInputVertexInterface(),
				FOutputVertexInterface(
					TOutputDataVertex<float>(METASOUND_GET_PARAM_NAME_AND_METADATA(OutFloat))
				)
			);

			return Interface;
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace ExampleNodeConfigurationPrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "WidgetConfigurableOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("WidgetExampleConfigurableNodeName", "A Widget Configurable Node"),
				METASOUND_LOCTEXT("WidgetExampleConfigurableNodeDescription", "A Node which shows how to make a configurable node with a custom details customization for yourself."),
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("WidgetExampleConfigurablePromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				GetVertexInterface(),
				{}
			};
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults)
		{
			using namespace ExampleNodeConfigurationPrivate;

			if (const FWidgetExampleOperatorData* ExampleConfig = CastOperatorData<const FWidgetExampleOperatorData>(InParams.Node.GetOperatorData().Get()))
			{
				return MakeUnique<FWidgetExampleConfigurableOperator>(
					StaticCastSharedPtr<const FWidgetExampleOperatorData>(InParams.Node.GetOperatorData())
				);
			}

			return nullptr;
		}

	private:
		// Contains configured float data
		TSharedPtr<const FWidgetExampleOperatorData> OperatorData;

		// Output
		FFloatWriteRef FloatOut;
	};

	using FWidgetExampleConfigurableNode = TNodeFacade<FWidgetExampleConfigurableOperator>;
	METASOUND_REGISTER_NODE_AND_CONFIGURATION(FWidgetExampleConfigurableNode, FMetaSoundWidgetExampleNodeConfiguration);
}

FMetaSoundExperimentalExampleNodeConfiguration::FMetaSoundExperimentalExampleNodeConfiguration()
: String("YES!")
, NumInputs(1)
, NumOutputs(1)
{
}

TInstancedStruct<FMetasoundFrontendClassInterface> FMetaSoundExperimentalExampleNodeConfiguration::OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const
{
	using namespace Metasound::Experimental::ExampleNodeConfigurationPrivate;
	// Override the interface based upon the number of inputs and outputs desired. 
	return TInstancedStruct<FMetasoundFrontendClassInterface>::Make(FMetasoundFrontendClassInterface::GenerateClassInterface(GetVertexInterface(NumInputs, NumOutputs)));
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundExperimentalExampleNodeConfiguration::GetOperatorData() const 
{
	// Any data the node configuration wishes to share with the operators can be produced here. 
	return MakeShared<Metasound::Experimental::FExampleOperatorData>(String);
}

FMetaSoundWidgetExampleNodeConfiguration::FMetaSoundWidgetExampleNodeConfiguration()
	: MyFloat(0.5f)
	, OperatorData(MakeShared<Metasound::Experimental::FWidgetExampleOperatorData>(MyFloat))
{
}

TSharedPtr<const Metasound::IOperatorData> FMetaSoundWidgetExampleNodeConfiguration::GetOperatorData() const
{
	OperatorData->MyFloat = MyFloat;
	return OperatorData;
}
#undef LOCTEXT_NAMESPACE // MetasoundExperimentalRuntime
