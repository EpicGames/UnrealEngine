// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExamplePolymorphicDataTypeNode.h"

#include "MetasoundExampleNodeConfiguration.h" 

#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

#include "MetasoundDataReference.h"
#include "MetasoundDataFactory.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"
#include "MetasoundReceiveNode.h"
#include "MetasoundTrigger.h"

#define LOCTEXT_NAMESPACE "MetasoundExperimentalRuntime"

namespace Metasound
{
	// 1. Let's define some simple types to illustrate the polymorphic types
	// A Simple base class
	struct FMyExampleBaseType
	{
		FName TypeName;
		const void* TypeId=nullptr;
		
		// Type Id+Name. (required for references to work).
		const FName& GetDataTypeName() const { return TypeName; }
		const void* GetDataTypeId() const { return TypeId; }
	
		FMyExampleBaseType(const FName InTypeName, const void* InTypeId) 
			: TypeName(InTypeName)
			, TypeId(InTypeId)
		{}
		virtual ~FMyExampleBaseType() = default;
		virtual void DoSomething() const { checkNoEntry(); }	// NOTE: This can't be pure virtual in order to register it with the type system.
		static TSharedRef<FMyExampleBaseType, ESPMode::NotThreadSafe> CreateNew(const FOperatorSettings&);
	};

	// 1a. A simple concrete type deriving from the base.
	struct FConcreteTypeA final : FMyExampleBaseType
	{
		static constexpr uint8 Token = 0;
		explicit FConcreteTypeA(const FOperatorSettings&)
			: FMyExampleBaseType(TEXT("ConcreteTypeA"), &Token)
		{}
		
		virtual void DoSomething() const override
		{
			UE_LOGF(LogTemp, Display, "DoSomething 'A' Version: %ls", *GetDataTypeName().ToString());			
		}
	};
	// 1b. A simple concrete type deriving from the base.
	struct FConcreteTypeB final : FMyExampleBaseType
	{
		static constexpr uint8 Token = 0;
		explicit FConcreteTypeB(const FOperatorSettings&)
			: FMyExampleBaseType(TEXT("ConcreteTypeB"), &Token)
		{
		}
		
		virtual void DoSomething() const override
		{
			UE_LOGF(LogTemp, Display, "DoSomething 'B' Version: %ls", *GetDataTypeName().ToString());
		}
	};
	
	TSharedRef<FMyExampleBaseType, ESPMode::NotThreadSafe> FMyExampleBaseType::CreateNew(const FOperatorSettings& InSettings)
	{
		// Need to make one of the concrete children. The "default" one.
		return MakeShared<FConcreteTypeA, ESPMode::NotThreadSafe>(InSettings);
	}

	// Declare the types using the metasound type macros.
	// This is normally in a header file so you can use it more than once place, but for this example we do it here.
	// This is the same as a normal datatype definition, expect for a new macro DECLARE_METASOUND_POLY_TYPE
	
	// 2. Declare the base class as Polymorphic type. (Declaring the base class as void if this is the base class itself, marking as Abstract).
	DECLARE_METASOUND_POLY_TYPE(FMyExampleBaseType, /* BaseClass*/ void, /*bIsAbstract */ true);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(FMyExampleBaseType, METASOUNDEXPERIMENTALRUNTIME_API);
	DEFINE_METASOUND_DATA_TYPE(FMyExampleBaseType, "MyBaseType");

	// 2a. Declare the concrete class a poly type (Declaring the base class as the abstract type above, and marking this as not-abstract).;
	DECLARE_METASOUND_POLY_TYPE(FConcreteTypeA, /* BaseClass*/ FMyExampleBaseType, /*bIsAbstract */ false);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(FConcreteTypeA, METASOUNDEXPERIMENTALRUNTIME_API);
	DEFINE_METASOUND_DATA_TYPE(FConcreteTypeA, "ConcreteTypeA");
	
	// 2b. Declare the other concrete class a poly type (Declaring the base class as the abstract type above, and marking this as not-abstract).;
	DECLARE_METASOUND_POLY_TYPE(FConcreteTypeB, /* BaseClass*/ FMyExampleBaseType, /*bIsAbstract */ false);
	DECLARE_METASOUND_DATA_REFERENCE_TYPES_NO_ALIASES(FConcreteTypeB, METASOUNDEXPERIMENTALRUNTIME_API);
	DEFINE_METASOUND_DATA_TYPE(FConcreteTypeB, "ConcreteTypeB");
	
	namespace ExamplePolymorphicDataTypeNodePrivate
	{
		// 3. Register the types somewhere in your module startup. This is a single .cpp so we do here.
		const bool bDoOnce = []() -> bool
		{
			// Register the types. (ideally in dependency order).
			REGISTER_METASOUND_POLY_TYPE(FMyExampleBaseType);
			REGISTER_METASOUND_POLY_TYPE(FConcreteTypeA);
			REGISTER_METASOUND_POLY_TYPE(FConcreteTypeB);
			return true;
		}();

		// 4. Define simple node interface.
		METASOUND_PARAM(InputRef, "Base", "Input Base type");
		METASOUND_PARAM(InputTrigger, "Trigger", "Input Trigger");
		
		// Create the node's vertex interface based upon the number of inputs and outputs
		// desired. 
		FVertexInterface GetVertexInterface()
		{
			// Define our simple example node interface.
			// 1. the *base* form of our new polymorphic type.
			// 2. A trigger
			
			FInputVertexInterface InputInterface;
			InputInterface.Add( TInputDataVertex<FMyExampleBaseType>{ METASOUND_GET_PARAM_NAME_AND_METADATA(InputRef) });	
			InputInterface.Add( TInputDataVertex<FTrigger>{ METASOUND_GET_PARAM_NAME_AND_METADATA(InputTrigger) });

			FOutputVertexInterface OutputInterface;
			return FVertexInterface
			{
				MoveTemp(InputInterface),
				MoveTemp(OutputInterface)
			};
		}
	}

	class FExamplePolymorphicTypedOperator : public TExecutableOperator<FExamplePolymorphicTypedOperator>
	{
		// 5. Node holds reference to the *base* type.
		TDataReadReference<FMyExampleBaseType> InputRef;
		FTriggerReadRef InputTrigger;
	public:

		// 6. References are passed into the constructor from the create function.
		FExamplePolymorphicTypedOperator(
			const TDataReadReference<FMyExampleBaseType>& InInputRef,
			const FTriggerReadRef& InInputTrigger)
				: InputRef(InInputRef)
				, InputTrigger(InInputTrigger)
		{
		}

		// 7a. Input Binding.
		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExamplePolymorphicDataTypeNodePrivate;
			InOutVertexData.BindReadVertex( METASOUND_GET_PARAM_NAME(InputRef), InputRef);
			InOutVertexData.BindReadVertex( METASOUND_GET_PARAM_NAME(InputTrigger), InputTrigger);
		}

		// 7b. Output Binding.
		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExamplePolymorphicDataTypeNodePrivate;
			// No outputs.
		}
		
		void Execute()
		{
			// When we get a Trigger.
			InputTrigger->ExecuteBlock(
				[](int32, int32) {},
				[this](int32, int32)
					{
						// ... call the virtual base function.
						InputRef->DoSomething();
					}
				);
		}

		void Reset(const IOperator::FResetParams& InParams)
		{
		}

		static FNodeClassMetadata GetNodeInfo()
		{
			using namespace ExamplePolymorphicDataTypeNodePrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "PolymorphicOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("ExamplePolymorphicNodeName", "An Example Polymoprhic Node"),	
				METASOUND_LOCTEXT("ExamplePolymorphicDescription", "An Example Polymorphic Node"),	
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("ExamplePolymorphicPromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				GetVertexInterface(),
				{}
			};
		}

		// Helper to make or Create the base type.
		static TDataReadReference<FMyExampleBaseType> GetOrCreateDefaultReadReference(const FBuildOperatorParams& InParams)
		{
			using namespace ExamplePolymorphicDataTypeNodePrivate;

			// If we're getting an existing one, just return the reference.
			if (InParams.InputData.IsVertexBound(METASOUND_GET_PARAM_NAME(InputRef)))
			{
				return InParams.InputData.GetDataReadReference<FMyExampleBaseType>(METASOUND_GET_PARAM_NAME(InputRef));		
			}

			// As this type is abstract. We need to make one of the base-types. How you handle this is up to you.
			// Call the CreateNewDerived.
			return TDataReadReference<FMyExampleBaseType>::CreateNew(InParams.OperatorSettings);		// In this case make Type A.
		}

		static TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) 
		{
			using namespace ExamplePolymorphicDataTypeNodePrivate;

			// GetOrCreate poly read reference.
			TDataReadReference<FMyExampleBaseType> Ref = GetOrCreateDefaultReadReference(InParams);

			// GetOrCreate a trigger reference.
			FTriggerReadRef Trigger = InParams.InputData.GetOrCreateDefaultDataReadReference<FTrigger>(METASOUND_GET_PARAM_NAME(InputTrigger), InParams.OperatorSettings);
			
			// Make operator.
			return MakeUnique<FExamplePolymorphicTypedOperator>(
					Ref,
					Trigger
				);
		}
	};

	using FExamplePolyNode = TNodeFacade<FExamplePolymorphicTypedOperator>;

	// The node extension must be registered along with the node. 
	METASOUND_REGISTER_NODE(FExamplePolyNode);

	// FMetaSoundWidgetExampleNodeConfiguration
}

#undef LOCTEXT_NAMESPACE // MetasoundExperimentalRuntime
