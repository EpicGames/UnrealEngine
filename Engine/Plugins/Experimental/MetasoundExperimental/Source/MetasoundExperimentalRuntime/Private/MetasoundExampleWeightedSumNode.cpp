// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExampleWeightedSumNode.h"

#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"

#include "MetasoundDynamicInterfaceNodeConfiguration.h"
#include "MetasoundDataFactory.h"
#include "MetasoundFacade.h"
#include "MetasoundOperatorData.h"
#include "MetasoundNodeRegistrationMacro.h"
#include "MetasoundExecutableOperator.h"
#include "MetasoundParamHelper.h"
#include "MetasoundPrimitives.h"


#define LOCTEXT_NAMESPACE "MetasoundExperimentalRuntime"

namespace Metasound::Experimental
{
	namespace ExampleWeightedSumNodePrivate
	{
		DEFINE_METASOUND_PARAM(InputValue, "Value", "A value to sum");
		DEFINE_METASOUND_PARAM(InputWeight, "Weight", "Relative weight for value when doing weighted sum.");
		DEFINE_METASOUND_PARAM(OutputSum, "WeightedSum", "Weightedum of all inputs");

		const FLazyName VariantName{"DataType"};

		FClassInterface MakeClassInterface()
		{
			return CreateClassInterface
			(
				FSubInterfaceDescription
				{
					"Inputs", // Name of sub interface
					1, 		// Minimum number of instances
					1000, 	// Maximum number of instances
					2 		// Default number of instances
				},

				FSubInterface
				{
					"Inputs", // Name of sub interface

					// Vertices in sub interface
					TInputVariantVertex<bool, int32, float>{METASOUND_GET_PARAM_NAME(InputValue), VariantName, METASOUND_GET_PARAM_METADATA(InputValue), false, 2, 3.f},
					TInputDataVertex<float>{METASOUND_GET_PARAM_NAME(InputWeight), METASOUND_GET_PARAM_METADATA(InputWeight), 1.f}
				},

				TOutputVariantVertex<bool, int32, float>{METASOUND_GET_PARAM_NAME(OutputSum), VariantName, METASOUND_GET_PARAM_METADATA(OutputSum)}
			);
		}
	}

	template<typename VariantType>
	class TExampleWeightedSumOperator : public TExecutableOperator<TExampleWeightedSumOperator<VariantType>>
	{
	public:
		TExampleWeightedSumOperator(
			TArray<TDataReadReference<float>> InInputWeights,
			TArray<TDataReadReference<VariantType>> InInputValues,
			TDataWriteReference<VariantType> InOutSum)
		: InputWeights(MoveTemp(InInputWeights))
		, InputValues(MoveTemp(InInputValues))
		, OutSum(MoveTemp(InOutSum))
		{
			check(InputWeights.Num() == InputValues.Num());
			Execute(); // Initialize OutSum
		}

		virtual void BindInputs(FInputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExampleWeightedSumNodePrivate;
			InOutVertexData.BindSubInterfaceVertices("Inputs", METASOUND_GET_PARAM_NAME(InputValue), InputValues);
			InOutVertexData.BindSubInterfaceVertices("Inputs", METASOUND_GET_PARAM_NAME(InputWeight), InputWeights);
		}

		virtual void BindOutputs(FOutputVertexInterfaceData& InOutVertexData) override
		{
			using namespace ExampleWeightedSumNodePrivate;
			InOutVertexData.BindVertex(METASOUND_GET_PARAM_NAME(OutputSum), OutSum);
		}

		void Execute()
		{
			float Sum = 0.f;
			const int32 NumInputs = InputWeights.Num();

			if constexpr (std::is_same_v<VariantType, bool>)
			{
				// Special handling for bool to treat it as 1=true and 0=false
				for (int32 i = 0; i < NumInputs; i++)
				{
					if (*InputValues[i])
					{
						Sum += *InputWeights[i];
					}
				}

				// If the output is a bool, make it true if the sum is greater
				// than or equal 0.5, false otherwise.
				if (Sum >= 0.5f)
				{
					*OutSum = true;
				}
				else
				{
					*OutSum = false;
				}
			}
			else
			{
				// Handle ints or floats
				for (int32 i = 0; i < NumInputs; i++)
				{
					Sum += *InputValues[i] * *InputWeights[i];
				}

				if constexpr (std::is_same_v<VariantType, int32>)
				{
					// Round if the output is an int
					*OutSum = FMath::RoundToInt(Sum);
				}
				else
				{
					// Assign if the output is a float
					*OutSum = Sum;
				}
			}

		}

		void Reset(const IOperator::FResetParams& InParams)
		{
			Execute(); // Reinitialize output value
		}

	private:
		TArray<TDataReadReference<float>> InputWeights;
		TArray<TDataReadReference<VariantType>> InputValues;
		TDataWriteReference<VariantType> OutSum;
	};

	// FNodeFacade doesn't handle the case of 1 node returning multiple different
	// operators very well. We could create some convenience FNodeFacade implementations
	// to make it easier for folks to write nodes which utilize variants.
	class FExampleWeightedSumNode : public FBasicNode
	{
		class FFactory : public IOperatorFactory
		{
		public:

			virtual TUniquePtr<IOperator> CreateOperator(const FBuildOperatorParams& InParams, FBuildResults& OutResults) override
			{
				using namespace ExampleWeightedSumNodePrivate;

				const FVertexInterface& NodeInterface = InParams.Node.GetVertexInterface();

				// Determine the variant data type by inspecting the node interface. We could look
				// at putting this information in the IOperatorData accessible through the Node.GetOperatorData(),
				// but this seems just as convenient.
				const FName VariantTypeName = NodeInterface.GetOutputVertex(METASOUND_GET_PARAM_NAME(OutputSum)).DataTypeName;
				if (VariantTypeName.IsNone() || VariantTypeName == GVariantDataTypeName)
				{
					// TODO: Should this be an error? Should we somehow skip nodes that
					// don't have an explicit type set for a variant?
					return {};
				}

				// These method gets/creates all the instances of the input depending on
				// the number of subinterfaces that exist on the interface.
				TArray<TDataReadReference<float>> InputWeights = InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<float>(
					"Inputs", 								// Name of sub interface
					METASOUND_GET_PARAM_NAME(InputWeight), 	// Name of vertex on sub interface
					InParams.OperatorSettings);

				// We need to handle the various possible datatypes of the inputs. This
				// is probably the most complicated scenario we'll encounter because
				// we are utilize both variants AND subinterfaces. The templatizing of the
				// operator is better than what we used to do because we are no longer
				// sharing lots of node registration data or repeating FNodeClassMetadata
				// codegen for each datatype and subinterface.
				//
				// There may be a way where we rewrite this to be more like TDataReadReference<bool, int32, float>
				// or where we templatize the minimum bits of the operator which would
				// still over reasonable memory savings.
				if (GetMetasoundDataTypeName<bool>() == VariantTypeName)
				{
					return MakeUnique<TExampleWeightedSumOperator<bool>>(
							MoveTemp(InputWeights),
							InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<bool>(
								"Inputs", 								// Name of sub interface
								METASOUND_GET_PARAM_NAME(InputValue), 	// Name of vertex on sub interface
								InParams.OperatorSettings),
							TDataWriteReference<bool>::CreateNew());
				}
				else if (GetMetasoundDataTypeName<int32>() == VariantTypeName)
				{
					return MakeUnique<TExampleWeightedSumOperator<int32>>(
							MoveTemp(InputWeights),
							InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<int32>(
								"Inputs", 								// Name of sub interface
								METASOUND_GET_PARAM_NAME(InputValue), 	// Name of vertex on sub interface
								InParams.OperatorSettings),
							TDataWriteReference<int32>::CreateNew());
				}
				else if (GetMetasoundDataTypeName<float>() == VariantTypeName)
				{
					return MakeUnique<TExampleWeightedSumOperator<float>>(
							MoveTemp(InputWeights),
							InParams.InputData.GetOrCreateDefaultSubInterfaceDataReadReferences<float>(
								"Inputs", 								// Name of sub interface
								METASOUND_GET_PARAM_NAME(InputValue), 	// Name of vertex on sub interface
								InParams.OperatorSettings),
							TDataWriteReference<float>::CreateNew());
				}
				else
				{
					// We shouldn't ever end up here. It means we have a varianttype which
					// we are not handling.
					checkNoEntry();
					return {};
				}
			}
		};

	public:
		FExampleWeightedSumNode(const FNodeData& InNodeData, const TSharedRef<const FNodeClassMetadata>& InClassMetadata)
		: FBasicNode(InNodeData, InClassMetadata)
		{
		}


		static FNodeClassMetadata CreateNodeClassMetadata()
		{
			using namespace ExampleWeightedSumNodePrivate;
			return FNodeClassMetadata
			{
				FNodeClassName{ "Experimental", "ConfigurableWeightedSumOperator", "" },
				1, // Major version
				0, // Minor version
				METASOUND_LOCTEXT("ExampleConfigurableWeightedSumNodeName", "Weighted Sum (Example Node)"),
				METASOUND_LOCTEXT("ExampleConfigurableWeightedSumNodeDescription", "A Node which shows how to make a configurable node for yourself using subinterfaces and variant vertices."),
				TEXT("UE"), // Author
				METASOUND_LOCTEXT("ExampleConfigurableWeightedSumPromptIfMissing", "Enable the MetaSoundExperimental Plugin"), // Prompt if missing
				MakeClassInterface(),
				{}
			};
		}

		virtual FOperatorFactorySharedRef GetDefaultOperatorFactory() const override
		{
			return MakeShared<FFactory>();
		}
	private:


	};

	// If using SubInterface or VariantVertex in the class interface of a node, the FMetasoundDynamicInterfaceNodeConfiguration will automatically
	// be applied.
	METASOUND_REGISTER_NODE(FExampleWeightedSumNode);
}



#undef LOCTEXT_NAMESPACE // MetasoundExperimentalRuntime
