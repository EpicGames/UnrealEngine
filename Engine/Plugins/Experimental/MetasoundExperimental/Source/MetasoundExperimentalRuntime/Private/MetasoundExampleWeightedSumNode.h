// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file is intentionally empty. The custom node configuration types
// that were here have been replaced by FMetaSoundDynamicInterfaceNodeConfiguration.
// The file is kept to avoid build system issues with the .generated.h
// reference until it can be fully removed.
