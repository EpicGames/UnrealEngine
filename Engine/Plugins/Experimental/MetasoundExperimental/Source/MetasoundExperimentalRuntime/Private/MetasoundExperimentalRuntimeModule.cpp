// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetasoundExperimentalRuntimeModule.h"

#include "MetasoundChannelAgnosticDataSchemas.h"
#include "MetasoundFrontendModuleRegistrationMacros.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

namespace MetasoundExperimentalRuntimePrivate
{
	static FString GetPluginContentPath()
	{
		if (const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MetasoundExperimental")); Plugin.IsValid())
		{
			return Plugin->GetContentDir();
		}

		// Some targets don't appear to return the plugin, so fall-back to this.
		const FString ContentPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Runtime/Experimental/MetasoundExperimental/Content"));;
		return ContentPath;
	}
}

void FMetasoundExperimentalRuntimeModule::StartupModule()
{
	FModuleManager::Get().LoadModule("AudioExperimentalRuntime");
	RegisteredDataSchemas.Append(Metasound::LoadChannelAgnosticDataSchemas(MetasoundExperimentalRuntimePrivate::GetPluginContentPath()));
	METASOUND_REGISTER_ITEMS_IN_MODULE
}

void FMetasoundExperimentalRuntimeModule::ShutdownModule()
{
	Metasound::UnloadChannelAgnosticDataSchemas(RegisteredDataSchemas);
	METASOUND_UNREGISTER_ITEMS_IN_MODULE
}

METASOUND_IMPLEMENT_MODULE_REGISTRATION_LIST
IMPLEMENT_MODULE(FMetasoundExperimentalRuntimeModule, MetasoundExperimentalRuntime);

