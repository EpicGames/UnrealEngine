// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocument.h"
#include "Templates/SharedPointer.h"
#include "UObject/PropertyText.h"

#include "MetasoundCatPannerNode.generated.h"

UCLASS()
class UMetasoundCatAzimuthalChannelOptionsHelper : public UObject
{
	GENERATED_BODY()

	UFUNCTION()
	static TArray<FPropertyTextFName> GetAzimuthalChannelOptions();
};

UENUM()
enum class ECatPannerMethod : uint8
{
	// Panning is done using equal-power law
	EqualPower,

	// Panning is done linearly
	Linear
};


USTRUCT()
struct FMetaSoundCatPannerNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundCatPannerNodeConfiguration() =  default;

	// What the CAT output type is. Only azimuthal discrete channel types supported.
	UPROPERTY(EditAnywhere, Category = General, meta = (GetOptions="MetasoundCatAzimuthalChannelOptionsHelper.GetAzimuthalChannelOptions"))
	FName PanToType = TEXT("Cat:Stereo2Dot0");

	// What panning method to use. Panning method defines the mapping function of gains between pair-wise channels.
	UPROPERTY(EditAnywhere, Category = General)
	ECatPannerMethod PanningMethod = ECatPannerMethod::EqualPower;
	
	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const override;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};


namespace Metasound
{
	// Normalization utility.	
	METASOUNDEXPERIMENTALRUNTIME_API float NormalizedAzimuthToDegrees(const float InNormalizedAzimuth);
}
