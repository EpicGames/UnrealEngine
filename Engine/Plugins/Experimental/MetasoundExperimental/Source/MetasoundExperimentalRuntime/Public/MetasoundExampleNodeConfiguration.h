// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundOperatorData.h"
#include "MetasoundFrontendDocument.h"

#include "MetasoundExampleNodeConfiguration.generated.h"

namespace Metasound::Experimental
{
	class FWidgetExampleOperatorData;
}

USTRUCT()
struct FMetaSoundExperimentalExampleNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundExperimentalExampleNodeConfiguration();

	UPROPERTY(EditAnywhere, Category = General)
	FString String;

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "1000"))
	uint32 NumInputs;

	UPROPERTY(EditAnywhere, Category = General, meta = (ClampMin = "1", ClampMax = "1000"))
	uint32 NumOutputs;

	/* Get the current interface for the class based upon the node extension */
	virtual TInstancedStruct<FMetasoundFrontendClassInterface> OverrideDefaultInterface(const FMetasoundFrontendClass& InNodeClass) const;

	/* Pass data down to the operator. */
	virtual TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;
};

namespace Metasound::Experimental
{
	class FWidgetExampleOperatorData : public TOperatorData<FWidgetExampleOperatorData>
	{
	public:
		static const FLazyName OperatorDataTypeName;

		FWidgetExampleOperatorData(const float& InFloat)
			: MyFloat(InFloat)
		{
		}

		float MyFloat;
	};
}

USTRUCT()
struct FMetaSoundWidgetExampleNodeConfiguration : public FMetaSoundFrontendNodeConfiguration
{
	GENERATED_BODY()

	FMetaSoundWidgetExampleNodeConfiguration();

	UPROPERTY(EditAnywhere, Category = General)
	float MyFloat;

	TSharedPtr<const Metasound::IOperatorData> GetOperatorData() const override;

private: 
	mutable TSharedPtr<Metasound::Experimental::FWidgetExampleOperatorData> OperatorData;
};
